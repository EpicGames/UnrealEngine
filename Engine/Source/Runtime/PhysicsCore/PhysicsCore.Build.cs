// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PhysicsCore: ModuleRules
{
	public PhysicsCore(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject"
			}
		);

		PublicDependencyModuleNames.AddRange(
		   new string[] {
				"DeveloperSettings"
		   }
	   );

		SetupModulePhysicsSupport(Target);
		

		// SetupModulePhysicsSupport adds a dependency on PhysicsCore, but we are PhysicsCore!
		PublicIncludePathModuleNames.Remove("PhysicsCore");
		PublicDependencyModuleNames.Remove("PhysicsCore");

		PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
	}
}
