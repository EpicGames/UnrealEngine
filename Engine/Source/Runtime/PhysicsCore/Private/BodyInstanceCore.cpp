// Copyright Epic Games, Inc. All Rights Reserved.

#include "BodyInstanceCore.h"
#include "BodySetupCore.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BodyInstanceCore)

int32 UseDeprecatedBehaviorUpdateMassScaleChanges = 0;
FAutoConsoleVariableRef CVarUseDeprecatedBehaviorUpdateMassScaleChanges(TEXT("p.UseDeprecatedBehaviorUpdateMassScaleChanges"),
	UseDeprecatedBehaviorUpdateMassScaleChanges, TEXT("Allows FBodyInstanceCore::bUpdateMassWhenScaleChanges to default to false. This has potential issues, but allows existing projects to retain old behavior"));

FBodyInstanceCore::FBodyInstanceCore()
: bSimulatePhysics(false)
, bOverrideMass(false)
, bEnableGravity(true)
, bUpdateKinematicFromSimulation(false)
, bGyroscopicTorqueEnabled(false)
, bAutoWeld(false)
, bStartAwake(true)
, bGenerateWakeEvents(false)
, bUpdateMassWhenScaleChanges(!UseDeprecatedBehaviorUpdateMassScaleChanges)
, bDirtyMassProps(false)
{
}

bool FBodyInstanceCore::ShouldInstanceSimulatingPhysics() const
{
	return bSimulatePhysics && BodySetup.IsValid() && BodySetup->GetCollisionTraceFlag() != ECollisionTraceFlag::CTF_UseComplexAsSimple;
}
