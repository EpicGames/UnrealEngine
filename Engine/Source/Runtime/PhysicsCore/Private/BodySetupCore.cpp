// Copyright Epic Games, Inc. All Rights Reserved.

#include "BodySetupCore.h"
#include "PhysicsSettingsCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BodySetupCore)

/** Helper for enum output... */
#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(txt) case txt: return TEXT(#txt);
#endif

const TCHAR* LexToString(ECollisionTraceFlag Enum)
{
	switch(Enum)
	{
		FOREACH_ENUM_ECOLLISIONTRACEFLAG(CASE_ENUM_TO_TEXT)
	}
	return TEXT("<Unknown ECollisionTraceFlag>");
}

const TCHAR* LexToString(EPhysicsType Enum)
{
	switch(Enum)
	{
		FOREACH_ENUM_EPHYSICSTYPE(CASE_ENUM_TO_TEXT)
	}
	return TEXT("<Unknown EPhysicsType>");
}

const TCHAR* LexToString(EBodyCollisionResponse::Type Enum)
{
	switch(Enum)
	{
		FOREACH_ENUM_EBODYCOLLISIONRESPONSE(CASE_ENUM_TO_TEXT)
	}
	return TEXT("<Unknown EBodyCollisionResponse>");
}

UBodySetupCore::UBodySetupCore(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CollisionTraceFlag = CTF_UseDefault;
}

TEnumAsByte<enum ECollisionTraceFlag> UBodySetupCore::GetCollisionTraceFlag() const
{
	TEnumAsByte<enum ECollisionTraceFlag> DefaultFlag = UPhysicsSettingsCore::Get()->DefaultShapeComplexity;
	return CollisionTraceFlag == ECollisionTraceFlag::CTF_UseDefault ? DefaultFlag : CollisionTraceFlag;
}
