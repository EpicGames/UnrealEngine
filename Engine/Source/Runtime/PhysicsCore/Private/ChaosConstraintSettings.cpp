// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosConstraintSettings.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "HAL/IConsoleManager.h"

namespace Chaos
{
	namespace JointConstraintDefaults
	{
		float JointStiffness = 1.0f;

		float LinearDriveStiffnessScale = 1.0f;
		float LinearDriveDampingScale = 1.0f;
		float AngularDriveStiffnessScale = 1.5f;
		float AngularDriveDampingScale = 1.5f;

		int SoftLinearForceMode = (int)EJointForceMode::Acceleration;
		float SoftLinearStiffnessScale = 1.5f;
		float SoftLinearDampingScale = 1.2f;

		int SoftAngularForceMode = (int)EJointForceMode::Acceleration;
		float SoftAngularStiffnessScale = 100000;
		float SoftAngularDampingScale = 1000; 

		float LinearBreakScale = 1.0f;
		float AngularBreakScale = 1.0f;

		FAutoConsoleVariableRef CVarJointStiffness(TEXT("p.Chaos.JointConstraint.JointStiffness"), JointStiffness, TEXT("Hard-joint solver stiffness."));
		FAutoConsoleVariableRef CVarLinearDriveStiffnessScale(TEXT("p.Chaos.JointConstraint.LinearDriveStiffnessScale"), LinearDriveStiffnessScale, TEXT("Conversion factor for Linear drive stiffness."));
		FAutoConsoleVariableRef CVarLinearDriveDampingScale(TEXT("p.Chaos.JointConstraint.LinaearDriveDampingScale"), LinearDriveDampingScale, TEXT("Conversion factor for Linear drive damping."));
		FAutoConsoleVariableRef CVarAngularDriveStiffnessScale(TEXT("p.Chaos.JointConstraint.AngularDriveStiffnessScale"), AngularDriveStiffnessScale, TEXT("Conversion factor for Angular drive stiffness."));
		FAutoConsoleVariableRef CVarAngularDriveDampingScale(TEXT("p.Chaos.JointConstraint.AngularDriveDampingScale"), AngularDriveDampingScale, TEXT("Conversion factor for Angular drive damping."));
		FAutoConsoleVariableRef CVarSoftLinearStiffnessScale(TEXT("p.Chaos.JointConstraint.SoftLinearStiffnessScale"), SoftLinearStiffnessScale, TEXT("Conversion factor for soft-joint stiffness."));
		FAutoConsoleVariableRef CVarSoftLinearDampingScale(TEXT("p.Chaos.JointConstraint.SoftLinearDampingScale"), SoftLinearDampingScale, TEXT("Conversion factor for soft-joint damping."));
		FAutoConsoleVariableRef CVarSoftLinearForceMode(TEXT("p.Chaos.JointConstraint.SoftLinearForceMode"), SoftLinearForceMode, TEXT("Soft Linear constraint force mode (0: Acceleration; 1: Force"));
		FAutoConsoleVariableRef CVarSoftAngularForceMode(TEXT("p.Chaos.JointConstraint.SoftAngularForceMode"), SoftAngularForceMode, TEXT("Soft Angular constraint force mode (0: Acceleration; 1: Force"));
		FAutoConsoleVariableRef CVarSoftAngularStiffnessScale(TEXT("p.Chaos.JointConstraint.SoftAngularStiffnessScale"), SoftAngularStiffnessScale, TEXT("Conversion factor for soft-joint stiffness."));
		FAutoConsoleVariableRef CVarSoftAngularDampingScale(TEXT("p.Chaos.JointConstraint.SoftAngularDampingScale"), SoftAngularDampingScale, TEXT("Conversion factor for soft-joint damping."));
		FAutoConsoleVariableRef CVarJointLinearBreakScale(TEXT("p.Chaos.JointConstraint.LinearBreakScale"), LinearBreakScale, TEXT("Conversion factory for Linear Break Theshold."));
		FAutoConsoleVariableRef CVarJointAngularBreakScale(TEXT("p.Chaos.JointConstraint.AngularBreakScale"), AngularBreakScale, TEXT("Conversion factory for Angular Break Theshold."));
	}

	FReal ConstraintSettings::JointStiffness() { return JointConstraintDefaults::JointStiffness;}
	FReal ConstraintSettings::LinearDriveStiffnessScale() { return JointConstraintDefaults::LinearDriveStiffnessScale;}
	FReal ConstraintSettings::LinearDriveDampingScale() { return JointConstraintDefaults::LinearDriveDampingScale;}
	FReal ConstraintSettings::AngularDriveStiffnessScale() { return JointConstraintDefaults::AngularDriveStiffnessScale;}
	FReal ConstraintSettings::AngularDriveDampingScale() { return JointConstraintDefaults::AngularDriveDampingScale;}
	int ConstraintSettings::SoftLinearForceMode() { return JointConstraintDefaults::SoftLinearForceMode;}
	FReal ConstraintSettings::SoftLinearStiffnessScale() { return JointConstraintDefaults::SoftLinearStiffnessScale;}
	FReal ConstraintSettings::SoftLinearDampingScale() { return JointConstraintDefaults::SoftLinearDampingScale;}
	int ConstraintSettings::SoftAngularForceMode() { return JointConstraintDefaults::SoftAngularForceMode;}
	FReal ConstraintSettings::SoftAngularStiffnessScale() { return JointConstraintDefaults::SoftAngularStiffnessScale;}
	FReal ConstraintSettings::SoftAngularDampingScale() { return JointConstraintDefaults::SoftAngularDampingScale;}
	FReal ConstraintSettings::LinearBreakScale() { return JointConstraintDefaults::LinearBreakScale;}
	FReal ConstraintSettings::AngularBreakScale() { return JointConstraintDefaults::AngularBreakScale;}

}
