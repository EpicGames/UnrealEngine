// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosEngineInterface.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsSettingsCore.h"
#include "PhysicsPublicCore.h"
#include "BodyInstanceCore.h"
#include "Chaos/ChaosScene.h"
#include "Chaos/KinematicTargets.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsObjectPhysicsCoreInterface.h"
#include "Chaos/Serialization/SerializationUtils.h"
#include "Chaos/Serialization/SerializedDataBuffer.h"
#include "Chaos/Serialization/SolverSerializer.h"
#include "Containers/AnsiString.h"
#include "Chaos/PhysicsObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosEngineInterface)

FPhysicsDelegatesCore::FOnUpdatePhysXMaterial FPhysicsDelegatesCore::OnUpdatePhysXMaterial;

#include "ChaosInterfaceWrapperCore.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/Sphere.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "CollisionShape.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/PBDSuspensionConstraintData.h"
#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/CastingUtilities.h"
#include "Math/UnitConversion.h"

namespace Chaos
{
	extern CHAOS_API int32 AccelerationStructureSplitStaticAndDynamic;
	extern CHAOS_API int32 AccelerationStructureIsolateQueryOnlyObjects;
	extern CHAOS_API int32 SyncKinematicOnGameThread;
}

bool bEnableChaosJointConstraints = true;
FAutoConsoleVariableRef CVarEnableChaosJointConstraints(TEXT("p.ChaosSolverEnableJointConstraints"), bEnableChaosJointConstraints, TEXT("Enable Joint Constraints defined within the Physics Asset Editor"));

bool bEnableChaosCollisionManager = true;
FAutoConsoleVariableRef CVarEnableChaosCollisionManager(TEXT("p.Chaos.Collision.EnableCollisionManager"), bEnableChaosCollisionManager, TEXT("Enable Chaos's Collision Manager for ignoring collisions between rigid bodies. [def:1]"));

bool FPhysicsConstraintReference_Chaos::IsValid() const
{
	return Constraint!=nullptr ? Constraint->IsValid() : false;
}
const Chaos::FImplicitObject& FPhysicsShapeReference_Chaos::GetGeometry() const
{
	check(IsValid()); return *Shape->GetGeometry();
}

FPhysicsGeometryCollection_Chaos::~FPhysicsGeometryCollection_Chaos() = default;
FPhysicsGeometryCollection_Chaos::FPhysicsGeometryCollection_Chaos(FPhysicsGeometryCollection_Chaos&& Steal) = default;

ECollisionShapeType FPhysicsGeometryCollection_Chaos::GetType() const
{
	return ChaosInterface::GetImplicitType(Geom);
}

const Chaos::FImplicitObject& FPhysicsGeometryCollection_Chaos::GetGeometry() const
{
	return Geom;
}

template<typename InnerType>
const InnerType& GetInnerGeometryChecked(const Chaos::FImplicitObject& InGeometry)
{
	const Chaos::FImplicitObject& InnerObject = Chaos::Utilities::CastHelper(InGeometry,
		[](const Chaos::FImplicitObject& CastGeom) -> const Chaos::FImplicitObject&
		{
			return CastGeom;
		});

	return InnerObject.GetObjectChecked<InnerType>();
}

const Chaos::TBox<Chaos::FReal,3>& FPhysicsGeometryCollection_Chaos::GetBoxGeometry() const
{
	return GetInnerGeometryChecked<Chaos::TBox<Chaos::FReal, 3>>(Geom);
}

const Chaos::TSphere<Chaos::FReal,3>&  FPhysicsGeometryCollection_Chaos::GetSphereGeometry() const
{
	return GetInnerGeometryChecked<Chaos::FSphere>(Geom);
}
const Chaos::FCapsule&  FPhysicsGeometryCollection_Chaos::GetCapsuleGeometry() const
{
	return GetInnerGeometryChecked<Chaos::FCapsule>(Geom);
}

const Chaos::FConvex& FPhysicsGeometryCollection_Chaos::GetConvexGeometry() const
{
	return GetInnerGeometryChecked<Chaos::FConvex>(Geom);
}

const Chaos::FTriangleMeshImplicitObject& FPhysicsGeometryCollection_Chaos::GetTriMeshGeometry() const
{
	return GetInnerGeometryChecked<Chaos::FTriangleMeshImplicitObject>(Geom);
}

FPhysicsGeometryCollection_Chaos::FPhysicsGeometryCollection_Chaos(const FPhysicsShapeReference_Chaos& InShape)
	: Geom(InShape.GetGeometry())
{
}

FPhysicsGeometryCollection_Chaos::FPhysicsGeometryCollection_Chaos(const FPhysicsGeometry& InGeom)
	: Geom(InGeom)
{
}

FPhysicsShapeAdapter_Chaos::FPhysicsShapeAdapter_Chaos(const FQuat& Rot,const FCollisionShape& CollisionShape)
	: GeometryRotation(Rot)
{
	switch(CollisionShape.ShapeType)
	{
	case ECollisionShape::Capsule:
	{
		const float CapsuleRadius = CollisionShape.GetCapsuleRadius();
		const float CapsuleHalfHeight = CollisionShape.GetCapsuleHalfHeight();
		if(CapsuleRadius < CapsuleHalfHeight)
		{
			const float UseHalfHeight = FMath::Max(CollisionShape.GetCapsuleAxisHalfLength(),FCollisionShape::MinCapsuleAxisHalfHeight());
			const FVector Bot = FVector(0.f,0.f,-UseHalfHeight);
			const FVector Top = FVector(0.f,0.f,UseHalfHeight);
			const float UseRadius = FMath::Max(CapsuleRadius,FCollisionShape::MinCapsuleRadius());
			Geometry = TRefCountPtr<FPhysicsGeometry>(new Chaos::FCapsule(Bot,Top,UseRadius));
		} else
		{
			// Use a sphere instead.
			const float UseRadius = FMath::Max(CapsuleRadius,FCollisionShape::MinSphereRadius());
			Geometry = TRefCountPtr<FPhysicsGeometry>(new Chaos::TSphere<Chaos::FReal,3>(Chaos::FVec3(0),UseRadius));
		}
		break;
	}
	case ECollisionShape::Box:
	{
		Chaos::FVec3 HalfExtents = CollisionShape.GetBox();
		HalfExtents.X = FMath::Max(HalfExtents.X,FCollisionShape::MinBoxExtent());
		HalfExtents.Y = FMath::Max(HalfExtents.Y,FCollisionShape::MinBoxExtent());
		HalfExtents.Z = FMath::Max(HalfExtents.Z,FCollisionShape::MinBoxExtent());

		Geometry = TRefCountPtr<FPhysicsGeometry>(new Chaos::TBox<Chaos::FReal,3>(-HalfExtents,HalfExtents));
		break;
	}
	case ECollisionShape::Sphere:
	{
		const float UseRadius = FMath::Max(CollisionShape.GetSphereRadius(),FCollisionShape::MinSphereRadius());
		Geometry = TRefCountPtr<FPhysicsGeometry>(new Chaos::TSphere<Chaos::FReal,3>(Chaos::FVec3(0),UseRadius));
		break;
	}
	default:
	ensure(false);
	break;
	}
}

FPhysicsShapeAdapter_Chaos::~FPhysicsShapeAdapter_Chaos() = default;

const FPhysicsGeometry& FPhysicsShapeAdapter_Chaos::GetGeometry() const
{
	return *Geometry;
}

FTransform FPhysicsShapeAdapter_Chaos::GetGeomPose(const FVector& Pos) const
{
	return FTransform(GeometryRotation,Pos);
}

const FQuat& FPhysicsShapeAdapter_Chaos::GetGeomOrientation() const
{
	return GeometryRotation;
}

void FChaosEngineInterface::AddActorToSolver(const FPhysicsActorHandle& Handle,Chaos::FPhysicsSolver* Solver)
{
	Solver->RegisterObject(Handle);
}


void FChaosEngineInterface::RemoveActorFromSolver(const FPhysicsActorHandle& Handle,Chaos::FPhysicsSolver* Solver)
{
	// Should we stop passing solver in? (need to check it's not null regardless in case proxy was never registered)
	if(Solver && Handle && Handle->GetSolverBase() == Solver)
	{
		Solver->UnregisterObject(Handle);
	}
	else
	{
		delete Handle;
	}
}

// Aggregate is not relevant for Chaos yet
FPhysicsAggregateReference_Chaos FChaosEngineInterface::CreateAggregate(int32 MaxBodies)
{
	// #todo : Implement
	FPhysicsAggregateReference_Chaos NewAggregate;
	return NewAggregate;
}

void FChaosEngineInterface::ReleaseAggregate(FPhysicsAggregateReference_Chaos& InAggregate) {}
int32 FChaosEngineInterface::GetNumActorsInAggregate(const FPhysicsAggregateReference_Chaos& InAggregate) { return 0; }
void FChaosEngineInterface::AddActorToAggregate_AssumesLocked(const FPhysicsAggregateReference_Chaos& InAggregate,const FPhysicsActorHandle& InActor) {}

Chaos::FChaosPhysicsMaterial::ECombineMode UToCCombineMode(EFrictionCombineMode::Type Mode)
{
	using namespace Chaos;
	switch(Mode)
	{
	case EFrictionCombineMode::Average: return FChaosPhysicsMaterial::ECombineMode::Avg;
	case EFrictionCombineMode::Min: return FChaosPhysicsMaterial::ECombineMode::Min;
	case EFrictionCombineMode::Multiply: return FChaosPhysicsMaterial::ECombineMode::Multiply;
	case EFrictionCombineMode::Max: return FChaosPhysicsMaterial::ECombineMode::Max;
	default: ensure(false);
	}

	return FChaosPhysicsMaterial::ECombineMode::Avg;
}

FPhysicsMaterialHandle FChaosEngineInterface::CreateMaterial(const UPhysicalMaterial* InMaterial)
{
	Chaos::FMaterialHandle NewHandle = Chaos::FPhysicalMaterialManager::Get().Create();

	return NewHandle;
}

void FChaosEngineInterface::UpdateMaterial(FPhysicsMaterialHandle& InHandle,UPhysicalMaterial* InMaterial)
{
	if(Chaos::FChaosPhysicsMaterial* Material = InHandle.Get())
	{
		Material->Friction = InMaterial->Friction;
		Material->StaticFriction = InMaterial->StaticFriction;
		Material->FrictionCombineMode = UToCCombineMode(InMaterial->FrictionCombineMode);
		Material->Restitution = InMaterial->Restitution;
		Material->RestitutionCombineMode = UToCCombineMode(InMaterial->RestitutionCombineMode);
		Material->Density = InMaterial->Density;
		Material->SleepingLinearThreshold = InMaterial->SleepLinearVelocityThreshold;
		Material->SleepingAngularThreshold = InMaterial->SleepAngularVelocityThreshold;
		Material->SleepCounterThreshold = InMaterial->SleepCounterThreshold;
		Material->Strength.TensileStrength = Chaos::MegaPascalToKgPerCmS2(InMaterial->Strength.TensileStrength);
		Material->Strength.CompressionStrength = Chaos::MegaPascalToKgPerCmS2(InMaterial->Strength.CompressionStrength);
		Material->Strength.ShearStrength = Chaos::MegaPascalToKgPerCmS2(InMaterial->Strength.ShearStrength);
		Material->DamageModifier.DamageThresholdMultiplier = InMaterial->DamageModifier.DamageThresholdMultiplier;
		Material->BaseFrictionImpulse = InMaterial->BaseFrictionImpulse;
		Material->SoftCollisionMode = (Chaos::EChaosPhysicsMaterialSoftCollisionMode)InMaterial->SoftCollisionMode;
		Material->SoftCollisionThickness = InMaterial->SoftCollisionThickness;
	}

	Chaos::FPhysicalMaterialManager::Get().UpdateMaterial(InHandle);
}

void FChaosEngineInterface::ReleaseMaterial(FPhysicsMaterialHandle& InHandle)
{
	Chaos::FPhysicalMaterialManager::Get().Destroy(InHandle);
}

void FChaosEngineInterface::SetUserData(const FPhysicsShapeHandle& InShape,void* InUserData)
{
	if(CHAOS_ENSURE(InShape.Shape))
	{
		InShape.Shape->SetUserData(InUserData);
	}
}


void FChaosEngineInterface::SetUserData(FPhysicsMaterialHandle& InHandle,void* InUserData)
{
	if(Chaos::FChaosPhysicsMaterial* Material = InHandle.Get())
	{
		Material->UserData = InUserData;
	}

	Chaos::FPhysicalMaterialManager::Get().UpdateMaterial(InHandle);
}

void FChaosEngineInterface::ReleaseMaterialMask(FPhysicsMaterialMaskHandle& InHandle)
{
	Chaos::FPhysicalMaterialManager::Get().Destroy(InHandle);
}


void* FChaosEngineInterface::GetUserData(const FPhysicsShapeHandle& InShape)
{
	if(ensure(InShape.Shape))
	{
		return InShape.Shape->GetUserData();
	}
	return nullptr;
}

int32 FChaosEngineInterface::GetNumShapes(const FPhysicsActorHandle& InHandle)
{
	// #todo : Implement
	return InHandle->GetGameThreadAPI().ShapesArray().Num();
}

void FChaosEngineInterface::ReleaseShape(const FPhysicsShapeHandle& InShape)
{
	check(!IsValid(InShape.ActorRef));
	//no need to delete because ownership is on actor. Is this an invalid assumption with the current API?
	//delete InShape.Shape;
}

void FChaosEngineInterface::AttachShape(const FPhysicsActorHandle& InActor,const FPhysicsShapeHandle& InNewShape)
{
	// #todo : Implement - this path is never used welding actually goes through FPhysInterface_Chaos::AddGeometry
	CHAOS_ENSURE(false);
}

void FChaosEngineInterface::DetachShape(const FPhysicsActorHandle& InActor,FPhysicsShapeHandle& InShape,bool bWakeTouching)
{
	if (CHAOS_ENSURE(InShape.Shape))
	{
		InActor->GetGameThreadAPI().RemoveShape(InShape.Shape, bWakeTouching);
	}
}

void FChaosEngineInterface::SetSmoothEdgeCollisionsEnabled_AssumesLocked(const FPhysicsActorHandle& InActor, const bool bSmoothEdgeCollisionsEnabled)
{
	InActor->GetGameThreadAPI().SetSmoothEdgeCollisionsEnabled(bSmoothEdgeCollisionsEnabled);
}

void FChaosEngineInterface::AddDisabledCollisionsFor_AssumesLocked(const TMap<FPhysicsActorHandle, TArray< FPhysicsActorHandle > >& InMap)
{
	if (bEnableChaosCollisionManager)
	{
		for (auto Elem : InMap)
		{
			FPhysicsActorHandle& ActorReference = Elem.Key;
			Chaos::FUniqueIdx ActorIndex = ActorReference->GetGameThreadAPI().UniqueIdx();

			Chaos::FPhysicsSolver* Solver = ActorReference->GetSolver<Chaos::FPhysicsSolver>();
			Chaos::FIgnoreCollisionManager& CollisionManager = Solver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager();
			int32 ExternalTimestamp = Solver->GetMarshallingManager().GetExternalTimestamp_External();
			Chaos::FIgnoreCollisionManager::FPendingMap& ActivationMap = CollisionManager.GetPendingActivationsForGameThread(ExternalTimestamp);

			if (ActivationMap.Contains(ActorIndex))
			{
				ActivationMap.Remove(ActorIndex);
			}

			TArray< Chaos::FUniqueIdx > DisabledCollisions;
			DisabledCollisions.Reserve(Elem.Value.Num());

			if (Chaos::FPBDRigidParticle* Rigid0 = ActorReference->GetParticle_LowLevel()->CastToRigidParticle())
			{
				for (auto Handle1 : Elem.Value)
				{
					if (Chaos::FPBDRigidParticle* Rigid1 = Handle1->GetParticle_LowLevel()->CastToRigidParticle())
					{
						DisabledCollisions.Add(Handle1->GetGameThreadAPI().UniqueIdx());
					}
				}
			}

			ActivationMap.Add(ActorIndex, DisabledCollisions);
		}
	}
}

void FChaosEngineInterface::RemoveDisabledCollisionsFor_AssumesLocked(TArray< FPhysicsActorHandle >& InPhysicsActors)
{
	if (bEnableChaosCollisionManager)
	{
		for (const FPhysicsActorHandle& ActorReference : InPhysicsActors)
		{
			Chaos::FUniqueIdx ActorIndex = ActorReference->GetGameThreadAPI().UniqueIdx();

			Chaos::FPhysicsSolver* Solver = ActorReference->GetSolver<Chaos::FPhysicsSolver>();
			Chaos::FIgnoreCollisionManager& CollisionManager = Solver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager();
			int32 ExternalTimestamp = Solver->GetMarshallingManager().GetExternalTimestamp_External();

			Chaos::FIgnoreCollisionManager::FDeactivationSet& DeactivationMap = CollisionManager.GetPendingDeactivationsForGameThread(ExternalTimestamp);
			DeactivationMap.Add(ActorReference->GetGameThreadAPI().UniqueIdx());
		}
	}
}

void FChaosEngineInterface::SetDisabled(const FPhysicsActorHandle& InPhysicsActor, bool bSetDisabled)
{
	InPhysicsActor->GetGameThreadAPI().SetDisabled(bSetDisabled);
}

bool FChaosEngineInterface::IsDisabled(const FPhysicsActorHandle& InPhysicsActor)
{
	return InPhysicsActor->GetGameThreadAPI().Disabled();
}

void FChaosEngineInterface::SetActorUserData_AssumesLocked(const FPhysicsActorHandle& InActorReference,FPhysicsUserData* InUserData)
{
	InActorReference->GetGameThreadAPI().SetUserData(InUserData);
}

bool FChaosEngineInterface::IsRigidBody(const FPhysicsActorHandle& InActorReference)
{
	return !IsStatic(InActorReference);
}

bool FChaosEngineInterface::IsDynamic(const FPhysicsActorHandle& InActorReference)
{
	return !IsStatic(InActorReference);
}

bool FChaosEngineInterface::IsStatic(const FPhysicsActorHandle& InActorReference)
{
	if(FChaosEngineInterface::IsValid(InActorReference))
	{
		return InActorReference->GetGameThreadAPI().ObjectState() == Chaos::EObjectStateType::Static;
	}

	return false;
}

bool FChaosEngineInterface::IsKinematic(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().ObjectState() == Chaos::EObjectStateType::Kinematic;
}

bool FChaosEngineInterface::IsKinematic_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return IsKinematic(InActorReference);
}

bool FChaosEngineInterface::IsSleeping(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().ObjectState() != Chaos::EObjectStateType::Dynamic;
}

bool FChaosEngineInterface::IsCcdEnabled(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().CCDEnabled();
}


bool FChaosEngineInterface::CanSimulate_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	// #todo : Implement
	return true;
}

float FChaosEngineInterface::GetMass_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().M();
}

void FChaosEngineInterface::SetSendsSleepNotifies_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bSendSleepNotifies)
{
	// # todo: Implement
	//check(bSendSleepNotifies == false);
}

void FChaosEngineInterface::ClearWakeEvent_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	InActorReference->GetGameThreadAPI().ClearEvents();
}

void FChaosEngineInterface::PutToSleep_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	// NOTE: We want to set the state whether or not it's asleep - if we currently think we're
	// asleep but the physics thread has queued up a wake event, then we still need to call
	// SetObjectState, so that this manual call will take priority.
	Chaos::FRigidBodyHandle_External& BodyHandle_External = InActorReference->GetGameThreadAPI();
	if (BodyHandle_External.ObjectState() == Chaos::EObjectStateType::Dynamic || BodyHandle_External.ObjectState() == Chaos::EObjectStateType::Sleeping)
	{
		InActorReference->GetGameThreadAPI().SetObjectState(Chaos::EObjectStateType::Sleeping);
	}

}

void FChaosEngineInterface::WakeUp_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	// NOTE: We want to set the state whether or not it's asleep - if we currently think we're
	// dynamic but the physics thread has queued up a sleep event, then we still need to call
	// SetObjectState, so that this manual call will take priority.
	Chaos::FRigidBodyHandle_External& BodyHandle_External = InActorReference->GetGameThreadAPI();
	if(BodyHandle_External.ObjectState() == Chaos::EObjectStateType::Dynamic || BodyHandle_External.ObjectState() == Chaos::EObjectStateType::Sleeping)
	{
		BodyHandle_External.SetObjectState(Chaos::EObjectStateType::Dynamic);
		BodyHandle_External.ClearEvents();
	}
}

void FChaosEngineInterface::SetIsKinematic_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bIsKinematic)
{
	using namespace Chaos;
	{
		const EObjectStateType NewState
			= bIsKinematic
			? EObjectStateType::Kinematic
			: EObjectStateType::Dynamic;

		bool AllowedToChangeToNewState = false;

		switch(InActorReference->GetGameThreadAPI().ObjectState())
		{
		case EObjectStateType::Kinematic:
		// from kinematic we can only go dynamic
		if(NewState == EObjectStateType::Dynamic)
		{
			AllowedToChangeToNewState = true;
		}
		break;

		case EObjectStateType::Dynamic:
		// from dynamic we can go to sleeping or to kinematic
		if(NewState == EObjectStateType::Kinematic)
		{
			AllowedToChangeToNewState = true;
		}
		break;

		case EObjectStateType::Sleeping:
		// this case was not allowed from CL 10506092, but it needs to in order for
		// FBodyInstance::SetInstanceSimulatePhysics to work on dynamic bodies which
		// have fallen asleep.
		if (NewState == EObjectStateType::Kinematic)
		{
			AllowedToChangeToNewState = true;
		}
		break;
		}

		if(AllowedToChangeToNewState)
		{
			InActorReference->GetGameThreadAPI().SetObjectState(NewState);
			if (NewState == Chaos::EObjectStateType::Kinematic)
			{
				// Reset velocity on a state change here
				InActorReference->GetGameThreadAPI().SetV(Chaos::FVec3((Chaos::FReal) 0));
				InActorReference->GetGameThreadAPI().SetW(Chaos::FVec3((Chaos::FReal) 0));
			}
		}
	}
}

void FChaosEngineInterface::SetCcdEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bIsCcdEnabled)
{
	InActorReference->GetGameThreadAPI().SetCCDEnabled(bIsCcdEnabled);
}

void FChaosEngineInterface::SetMACDEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bIsMACDEnabled)
{
	InActorReference->GetGameThreadAPI().SetMACDEnabled(bIsMACDEnabled);
}

void FChaosEngineInterface::SetPartialIslandSleepAllowed_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bIsPartialIslandSleepAllowed)
{
	InActorReference->GetGameThreadAPI().SetPartialIslandSleepAllowed(bIsPartialIslandSleepAllowed);
}

void FChaosEngineInterface::SetPositionSolverIterationCount_AssumesLocked(const FPhysicsActorHandle& InActorReference, int32 PositionSolverIterationCount)
{
	InActorReference->GetGameThreadAPI().SetPositionSolverIterationCount(PositionSolverIterationCount);
}

void FChaosEngineInterface::SetVelocitySolverIterationCount_AssumesLocked(const FPhysicsActorHandle& InActorReference, int32 VelocitySolverIterationCount)
{
	InActorReference->GetGameThreadAPI().SetVelocitySolverIterationCount(VelocitySolverIterationCount);
}

void FChaosEngineInterface::SetProjectionSolverIterationCount_AssumesLocked(const FPhysicsActorHandle& InActorReference, int32 ProjectionSolverIterationCount)
{
	InActorReference->GetGameThreadAPI().SetProjectionSolverIterationCount(ProjectionSolverIterationCount);
}


void FChaosEngineInterface::SetIgnoreAnalyticCollisions_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bIgnoreAnalyticCollisions)
{
	InActorReference->GetGameThreadAPI().SetIgnoreAnalyticCollisions(bIgnoreAnalyticCollisions);
}

FTransform FChaosEngineInterface::GetGlobalPose_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return Chaos::FRigidTransform3(InActorReference->GetGameThreadAPI().X(),InActorReference->GetGameThreadAPI().R());
}

FTransform FChaosEngineInterface::GetTransform_AssumesLocked(const FPhysicsActorHandle& InRef,bool bForceGlobalPose /*= false*/)
{
	if(!bForceGlobalPose)
	{
		if(IsDynamic(InRef))
		{
			if(HasKinematicTarget_AssumesLocked(InRef))
			{
				return GetKinematicTarget_AssumesLocked(InRef);
			}
		}
	}

	return GetGlobalPose_AssumesLocked(InRef);
}

bool FChaosEngineInterface::HasKinematicTarget_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return IsStatic(InActorReference);
}

FTransform FChaosEngineInterface::GetKinematicTarget_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	// #todo : Implement
	//for now just use global pose
	return FChaosEngineInterface::GetGlobalPose_AssumesLocked(InActorReference);
}

FVector FChaosEngineInterface::GetLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return InActorReference->GetGameThreadAPI().V();
	}

	return FVector(0);
}

void FChaosEngineInterface::SetLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InNewVelocity,bool bAutoWake)
{
	// TODO: Implement bAutoWake == false.
	// For now we don't support auto-awake == false.
	// This feature is meant to detect when velocity change small
	// and the velocity is nearly zero, and to not wake up the
	// body in that case.
	ensure(bAutoWake);

	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetV(InNewVelocity);
	}
}

FVector FChaosEngineInterface::GetAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return InActorReference->GetGameThreadAPI().W();
	}

	return FVector(0);
}

void FChaosEngineInterface::SetAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InNewAngularVelocity,bool bAutoWake)
{
	// TODO: Implement bAutoWake == false.
	ensure(bAutoWake);

	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetW(InNewAngularVelocity);
	}
}

float FChaosEngineInterface::GetMaxAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return FMath::Sqrt(InActorReference->GetGameThreadAPI().GetMaxAngularSpeedSq());
	}

	return TNumericLimits<float>::Max();
}

float FChaosEngineInterface::GetMaxLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return FMath::Sqrt(InActorReference->GetGameThreadAPI().GetMaxLinearSpeedSq());
	}

	return TNumericLimits<float>::Max();
}

void FChaosEngineInterface::SetMaxAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InMaxAngularVelocityRadians)
{
	// We're about to square the input so we clamp to this maximum
	static const float MaxInput = FMath::Sqrt(TNumericLimits<float>::Max());

	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetMaxAngularSpeedSq(InMaxAngularVelocityRadians > MaxInput ? TNumericLimits<float>::Max() : InMaxAngularVelocityRadians * InMaxAngularVelocityRadians);
	}
}

void FChaosEngineInterface::SetMaxLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference, float InMaxLinearVelocity)
{
	// We're about to square the input so we clamp to this maximum
	static const float MaxInput = FMath::Sqrt(TNumericLimits<float>::Max());

	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetMaxLinearSpeedSq(InMaxLinearVelocity > MaxInput ? TNumericLimits<float>::Max() : InMaxLinearVelocity * InMaxLinearVelocity);
	}
}

float FChaosEngineInterface::GetMaxDepenetrationVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return FMath::Sqrt(InActorReference->GetGameThreadAPI().GetInitialOverlapDepenetrationVelocity());
	}

	return 0;
}

void FChaosEngineInterface::SetMaxDepenetrationVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InMaxDepenetrationVelocity)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetInitialOverlapDepenetrationVelocity(InMaxDepenetrationVelocity);
	}
}

FVector FChaosEngineInterface::GetWorldVelocityAtPoint_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InPoint)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		const Chaos::FRigidBodyHandle_External& Body_External = InActorReference->GetGameThreadAPI();
		if(Body_External.CanTreatAsKinematic())
		{
			const bool bIsRigid = Body_External.CanTreatAsRigid();
			const Chaos::FVec3 COM = bIsRigid ? Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(&Body_External) : (Chaos::FVec3)Chaos::FParticleUtilitiesGT::GetActorWorldTransform(&Body_External).GetTranslation();
			const Chaos::FVec3 Diff = InPoint - COM;
			return Body_External.V() - Chaos::FVec3::CrossProduct(Diff, Body_External.W());
		}
	}
	return FVector(0);
}

FVector FChaosEngineInterface::GetWorldVelocityAtPoint_AssumesLocked(const Chaos::FRigidBodyHandle_Internal* Body_Internal, const FVector& InPoint)
{
	const Chaos::FVec3 COM = Body_Internal->CanTreatAsRigid() ? Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(Body_Internal) : (Chaos::FVec3)Chaos::FParticleUtilitiesGT::GetActorWorldTransform(Body_Internal).GetTranslation();
	const Chaos::FVec3 Diff = InPoint - COM;
	return Body_Internal->V() - Chaos::FVec3::CrossProduct(Diff, Body_Internal->W());
}

FTransform FChaosEngineInterface::GetComTransform_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return Chaos::FParticleUtilitiesGT::GetCoMWorldTransform(&InActorReference->GetGameThreadAPI());
	}
	return FTransform();
}

FTransform FChaosEngineInterface::GetComTransformLocal_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return FTransform(InActorReference->GetGameThreadAPI().RotationOfMass(),InActorReference->GetGameThreadAPI().CenterOfMass());
	}
	return FTransform();
}

void FChaosEngineInterface::SetComTransformLocal_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FTransform& InTransform)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetRotationOfMass(InTransform.GetRotation());
		InActorReference->GetGameThreadAPI().SetCenterOfMass(InTransform.GetTranslation());
	}
}

FVector FChaosEngineInterface::GetLocalInertiaTensor_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		return FVector(InActorReference->GetGameThreadAPI().I());
	}
	return FVector::ZeroVector;
}

void FChaosEngineInterface::SetLocalInertiaTensor_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& InInertiaTensor)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetI(InInertiaTensor);
	}
}

FBox FChaosEngineInterface::GetBounds_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	using namespace Chaos;
	const Chaos::FRigidBodyHandle_External& Body_External = InActorReference->GetGameThreadAPI();

	const FTransform WorldTM(Body_External.R(), Body_External.X());
	return GetBounds_AssumesLocked(InActorReference, WorldTM);
}

FBox FChaosEngineInterface::GetBounds_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FTransform& InTransform)
{
	using namespace Chaos;
	const Chaos::FRigidBodyHandle_External& Body_External = InActorReference->GetGameThreadAPI();
	if (const FImplicitObjectRef Geometry = Body_External.GetGeometry())
	{
		if (Geometry->HasBoundingBox())
		{
			const FAABB3 LocalBounds = Geometry->BoundingBox();
			const FRigidTransform3 WorldTM(InTransform);
			const FAABB3 WorldBounds = LocalBounds.TransformedAABB(WorldTM);
			return FBox(WorldBounds.Min(), WorldBounds.Max());
		}
	}

	return FBox(EForceInit::ForceInitToZero);
}

FBox FChaosEngineInterface::GetBoundsLocal_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	using namespace Chaos;
	const Chaos::FRigidBodyHandle_External& Body_External = InActorReference->GetGameThreadAPI();
	if (const FImplicitObjectRef Geometry = Body_External.GetGeometry())
	{
		if (Geometry->HasBoundingBox())
		{
			const FAABB3 LocalBounds = Geometry->BoundingBox();
			return FBox(LocalBounds.Min(), LocalBounds.Max());
		}
	}

	return FBox(EForceInit::ForceInitToZero);
}

void FChaosEngineInterface::SetLinearDamping_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InDrag)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetLinearEtherDrag(InDrag);
	}
}

void FChaosEngineInterface::SetAngularDamping_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InDamping)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		InActorReference->GetGameThreadAPI().SetAngularEtherDrag(InDamping);
	}
}

template<typename BodyHandleType>
struct FChaosStateOps
{
	static void AddImpulse(BodyHandleType& BodyHandle, const FVector& InForce)
	{
		BodyHandle.SetLinearImpulse(BodyHandle.LinearImpulse() + InForce, /*bIsVelocity=*/false);
	}

	static void AddAngularImpulseInRadians(BodyHandleType& BodyHandle, const FVector& InTorque)
	{
		BodyHandle.SetAngularImpulse(BodyHandle.AngularImpulse() + InTorque, /*bIsVelocity=*/false);
	}

	static void AddVelocity(BodyHandleType& BodyHandle, const FVector& InVelocityDelta)
	{
		AddImpulse(BodyHandle, BodyHandle.M() * InVelocityDelta);
	}

	static void AddAngularVelocityInRadians(BodyHandleType& BodyHandle, const FVector& InAngularVelocityDeltaRad)
	{
		const Chaos::FMatrix33 WorldI = Chaos::FParticleUtilitiesXR::GetWorldInertia(&BodyHandle);
		AddAngularImpulseInRadians(BodyHandle, WorldI * InAngularVelocityDeltaRad);
	}

	static void AddImpulseAtLocation(BodyHandleType& BodyHandle, const FVector& InImpulse, const FVector& InLocation)
	{
		const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(&BodyHandle);
		const Chaos::FVec3 AngularImpulse = Chaos::FVec3::CrossProduct(InLocation - WorldCOM, InImpulse);
		AddImpulse(BodyHandle, InImpulse);
		AddAngularImpulseInRadians(BodyHandle, AngularImpulse);
	}

	static void AddVelocityChangeImpulseAtLocation(BodyHandleType& BodyHandle, const FVector& InVelocityDelta, const FVector& InLocation)
	{
		AddImpulseAtLocation(BodyHandle, BodyHandle.M() * InVelocityDelta, InLocation);
	}

	static void AddRadialImpulse(BodyHandleType& BodyHandle, const FVector& InOrigin, float InRadius, float InStrength, ERadialImpulseFalloff InFalloff, bool bInVelChange)
	{
		const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(&BodyHandle);
		const Chaos::FVec3 OriginToActor = WorldCOM - InOrigin;
		const Chaos::FReal OriginToActorDistance = OriginToActor.Size();
		if (OriginToActorDistance < InRadius)
		{
			Chaos::FVec3 FinalImpulse = FVector::ZeroVector;
			if (OriginToActorDistance > 0)
			{
				const Chaos::FVec3 OriginToActorNorm = OriginToActor / OriginToActorDistance;

				if (InFalloff == ERadialImpulseFalloff::RIF_Constant)
				{
					FinalImpulse = OriginToActorNorm * InStrength;
				}
				else if (InFalloff == ERadialImpulseFalloff::RIF_Linear)
				{
					const Chaos::FReal DistanceOverlapping = InRadius - OriginToActorDistance;
					if (DistanceOverlapping > 0)
					{
						FinalImpulse = OriginToActorNorm * FMath::Lerp(0.0f, InStrength, DistanceOverlapping / InRadius);
					}
				}
				else
				{
					// Unimplemented falloff type
					ensure(false);
				}
			}
			else
			{
				// Sphere and actor center are coincident, just pick a direction and apply maximum strength impulse.
				FinalImpulse = FVector::ForwardVector * InStrength;
			}

			if (bInVelChange)
			{
				AddVelocity(BodyHandle, FinalImpulse);
			}
			else
			{
				AddImpulse(BodyHandle, FinalImpulse);
			}
		}
	}

	static void AddForce(BodyHandleType& BodyHandle, const FVector& Force, bool bAllowSubstepping, bool bAccelChange)
	{		
		if (!Force.IsZero())
		{
			constexpr bool bAllowEvents = true;
			BodyHandle.SetObjectState(Chaos::EObjectStateType::Dynamic, bAllowEvents);
		}

		if (bAccelChange)
		{
			const Chaos::FReal Mass = BodyHandle.M();
			const Chaos::FVec3 Acceleration = Force * Mass;
			BodyHandle.AddForce(Acceleration);
		}
		else
		{
			BodyHandle.AddForce(Force);
		}
	}

	static void AddForceAtPosition(BodyHandleType& BodyHandle, const FVector& Force, const FVector& Position, bool bAllowSubstepping, bool bIsLocalForce /*= false*/)
	{
		const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(&BodyHandle);

		constexpr bool bAllowEvents = true;
		BodyHandle.SetObjectState(Chaos::EObjectStateType::Dynamic, bAllowEvents);

		if (bIsLocalForce)
		{
			const Chaos::FRigidTransform3 CurrentTransform = Chaos::FParticleUtilitiesGT::GetActorWorldTransform(&BodyHandle);
			const Chaos::FVec3 WorldPosition = CurrentTransform.TransformPosition(Position);
			const Chaos::FVec3 WorldForce = CurrentTransform.TransformVector(Force);
			const Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(WorldPosition - WorldCOM, WorldForce);
			BodyHandle.AddForce(WorldForce);
			BodyHandle.AddTorque(WorldTorque);
		}
		else
		{
			const Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(Position - WorldCOM, Force);
			BodyHandle.AddForce(Force);
			BodyHandle.AddTorque(WorldTorque);
		}
	}

	static void AddRadialForce(BodyHandleType& BodyHandle, const FVector& Origin, const float Radius, const float Strength, const uint8 Falloff, bool bAccelChange, bool bAllowSubstepping)
	{
		Chaos::EObjectStateType ObjectState = BodyHandle.ObjectState();
		if (CHAOS_ENSURE(ObjectState == Chaos::EObjectStateType::Dynamic || ObjectState == Chaos::EObjectStateType::Sleeping))
		{
			const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(&BodyHandle);

			Chaos::FVec3 Direction = WorldCOM - Origin;
			const Chaos::FReal Distance = Direction.Size();
			if (Distance > Radius)
			{
				return;
			}

			constexpr bool bAllowEvents = true;
			BodyHandle.SetObjectState(Chaos::EObjectStateType::Dynamic, bAllowEvents);

			if (Distance < 1e-4)
			{
				Direction = Chaos::FVec3(1, 0, 0);
			}
			else
			{
				Direction = Direction.GetUnsafeNormal();
			}
			Chaos::FVec3 Force(0, 0, 0);
			CHAOS_ENSURE(Falloff < RIF_MAX);
			if (Falloff == ERadialImpulseFalloff::RIF_Constant)
			{
				Force = Strength * Direction;
			}
			if (Falloff == ERadialImpulseFalloff::RIF_Linear)
			{
				Force = (Radius - Distance) / Radius * Strength * Direction;
			}
			if (bAccelChange)
			{
				const Chaos::FReal Mass = BodyHandle.M();
				const Chaos::FVec3 Acceleration = Force * Mass;
				BodyHandle.AddForce(Acceleration);
			}
			else
			{
				BodyHandle.AddForce(Force);
			}
		}
	}

	static void AddTorque(BodyHandleType& BodyHandle, const FVector& Torque, bool bAllowSubstepping, bool bAccelChange)
	{
		Chaos::EObjectStateType ObjectState = BodyHandle.ObjectState();
		if (CHAOS_ENSURE(ObjectState == Chaos::EObjectStateType::Dynamic || ObjectState == Chaos::EObjectStateType::Sleeping))
		{
			if (bAccelChange)
			{
				BodyHandle.AddTorque(Chaos::FParticleUtilitiesXR::GetWorldInertia(&BodyHandle) * Torque);
			}
			else
			{
				BodyHandle.AddTorque(Torque);
			}
		}
	}
};

void FChaosEngineInterface::AddImpulse_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InForce, bool bIsInternal)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddImpulse(*InActorReference->GetPhysicsThreadAPI(), InForce);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddImpulse(InActorReference->GetGameThreadAPI(), InForce);
		}
	}
}

void FChaosEngineInterface::AddAngularImpulseInRadians_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InTorque, bool bIsInternal)
{
	if(ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddAngularImpulseInRadians(*InActorReference->GetPhysicsThreadAPI(), InTorque);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddAngularImpulseInRadians(InActorReference->GetGameThreadAPI(), InTorque);
		}
	}
}

void FChaosEngineInterface::AddVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InVelocityDelta, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddVelocity(*InActorReference->GetPhysicsThreadAPI(), InVelocityDelta);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddVelocity(InActorReference->GetGameThreadAPI(), InVelocityDelta);
		}
	}
}

void FChaosEngineInterface::AddAngularVelocityInRadians_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InAngularVelocityDeltaRad, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddAngularVelocityInRadians(*InActorReference->GetPhysicsThreadAPI(), InAngularVelocityDeltaRad);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddAngularVelocityInRadians(InActorReference->GetGameThreadAPI(), InAngularVelocityDeltaRad);
		}
	}
}

void FChaosEngineInterface::AddImpulseAtLocation_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InImpulse,const FVector& InLocation, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddImpulseAtLocation(*InActorReference->GetPhysicsThreadAPI(), InImpulse, InLocation);
			//UE_LOGF(LogTemp, Warning, "AddImpulseAtLocation_AssumesLocked Impulse = %ls | State = %d", *InImpulse.ToString(), InActorReference->GetPhysicsThreadAPI()->ObjectState());
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddImpulseAtLocation(InActorReference->GetGameThreadAPI(), InImpulse, InLocation);
		}
	}
}

void FChaosEngineInterface::AddVelocityChangeImpulseAtLocation_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& InVelocityDelta, const FVector& InLocation, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddVelocityChangeImpulseAtLocation(*InActorReference->GetPhysicsThreadAPI(), InVelocityDelta, InLocation);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddVelocityChangeImpulseAtLocation(InActorReference->GetGameThreadAPI(), InVelocityDelta, InLocation);
		}
	}
}

void FChaosEngineInterface::AddRadialImpulse_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InOrigin,float InRadius,float InStrength,ERadialImpulseFalloff InFalloff,bool bInVelChange, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)) && ensure(InActorReference->GetGameThreadAPI().CanTreatAsRigid()))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddRadialImpulse(*InActorReference->GetPhysicsThreadAPI(), InOrigin, InRadius, InStrength, InFalloff, bInVelChange);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddRadialImpulse(InActorReference->GetGameThreadAPI(), InOrigin, InRadius, InStrength, InFalloff, bInVelChange);
		}
	}
}

void FChaosEngineInterface::AddForce_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Force, bool bAllowSubstepping, bool bAccelChange, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddForce(*InActorReference->GetPhysicsThreadAPI(), Force, bAllowSubstepping, bAccelChange);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddForce(InActorReference->GetGameThreadAPI(), Force, bAllowSubstepping, bAccelChange);
		}
	}
}

void FChaosEngineInterface::AddForceAtPosition_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Force, const FVector& Position, bool bAllowSubstepping, bool bIsLocalForce, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddForceAtPosition(*InActorReference->GetPhysicsThreadAPI(), Force, Position, bAllowSubstepping, bIsLocalForce);
			//UE_LOGF(LogTemp, Warning, "AddForceAtPosition_AssumesLocked Force = %ls | State = %d", *Force.ToString(), InActorReference->GetPhysicsThreadAPI()->ObjectState());
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddForceAtPosition(InActorReference->GetGameThreadAPI(), Force, Position,  bAllowSubstepping, bIsLocalForce);
		}
	}
}

void FChaosEngineInterface::AddRadialForce_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Origin, const float Radius, const float Strength, const uint8 Falloff, bool bAccelChange, bool bAllowSubstepping, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddRadialForce(*InActorReference->GetPhysicsThreadAPI(), Origin, Radius, Strength, Falloff, bAccelChange, bAllowSubstepping);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddRadialForce(InActorReference->GetGameThreadAPI(), Origin, Radius, Strength, Falloff, bAccelChange, bAllowSubstepping);
		}
	}
}

void FChaosEngineInterface::AddTorque_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Torque, bool bAllowSubstepping, bool bAccelChange, bool bIsInternal)
{
	if (ensure(FChaosEngineInterface::IsValid(InActorReference)))
	{
		if (bIsInternal)
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_Internal>::AddTorque(*InActorReference->GetPhysicsThreadAPI(), Torque, bAllowSubstepping, bAccelChange);
		}
		else
		{
			FChaosStateOps<Chaos::FRigidBodyHandle_External>::AddTorque(InActorReference->GetGameThreadAPI(), Torque, bAllowSubstepping, bAccelChange);
		}
	}
}

bool FChaosEngineInterface::IsGravityEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().GravityEnabled();
}
void FChaosEngineInterface::SetGravityEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bEnabled)
{
	InActorReference->GetGameThreadAPI().SetGravityEnabled(bEnabled);
}

PHYSICSCORE_API int32 FChaosEngineInterface::GetGravityGroupIndex_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().GravityGroupIndex();
}

void FChaosEngineInterface::SetGravityGroupIndex_AssumesLocked(const FPhysicsActorHandle& InActorReference, uint32 Index)
{
	InActorReference->GetGameThreadAPI().SetGravityGroupIndex(Index);
}

bool FChaosEngineInterface::GetUpdateKinematicFromSimulation_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().UpdateKinematicFromSimulation();
}
void FChaosEngineInterface::SetUpdateKinematicFromSimulation_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bUpdateKinematicFromSimulation)
{
	InActorReference->GetGameThreadAPI().SetUpdateKinematicFromSimulation(bUpdateKinematicFromSimulation);
}

bool FChaosEngineInterface::GetGyroscopicTorqueEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().GyroscopicTorqueEnabled();
}
void FChaosEngineInterface::SetGyroscopicTorqueEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bGyroscopicTorqueEnabled)
{
	InActorReference->GetGameThreadAPI().SetGyroscopicTorqueEnabled(bGyroscopicTorqueEnabled);
}

void FChaosEngineInterface::SetOneWayInteraction_AssumesLocked(const FPhysicsActorHandle& InHandle, bool InOneWayInteraction)
{
	InHandle->GetGameThreadAPI().SetOneWayInteraction(InOneWayInteraction);
}

float FChaosEngineInterface::GetSleepEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return 0;
}
void FChaosEngineInterface::SetSleepEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InEnergyThreshold)
{
}

void FChaosEngineInterface::SetSleepThresholdMultiplier_AssumesLocked(const FPhysicsActorHandle& InActorReference, float InThresholdMultiplier)
{
	return InActorReference->GetGameThreadAPI().SetSleepThresholdMultiplier(InThresholdMultiplier);
}

void FChaosEngineInterface::SetMass_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InMass)
{
	Chaos::FRigidBodyHandle_External& Body_External = InActorReference->GetGameThreadAPI();
	Body_External.SetM(InMass);
	if(CHAOS_ENSURE(!FMath::IsNearlyZero(InMass)))
	{
		Body_External.SetInvM(1./InMass);
	} else
	{
		Body_External.SetInvM(0);
	}
}

void FChaosEngineInterface::SetMassSpaceInertiaTensor_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InTensor)
{
	if(CHAOS_ENSURE(!FMath::IsNearlyZero(InTensor.X)) && CHAOS_ENSURE(!FMath::IsNearlyZero(InTensor.Y)) && CHAOS_ENSURE(!FMath::IsNearlyZero(InTensor.Z)))
	{
		Chaos::FRigidBodyHandle_External& Body_External = InActorReference->GetGameThreadAPI();
		Body_External.SetI(Chaos::TVec3<Chaos::FRealSingle>(InTensor.X,InTensor.Y,InTensor.Z));
		Body_External.SetInvI(Chaos::TVec3<Chaos::FRealSingle>(1./InTensor.X,1./InTensor.Y,1./InTensor.Z));
	}
}

void FChaosEngineInterface::SetComLocalPose_AssumesLocked(const FPhysicsActorHandle& InHandle,const FTransform& InComLocalPose)
{
	Chaos::FRigidBodyHandle_External& Body_External = InHandle->GetGameThreadAPI();
	Body_External.SetCenterOfMass(InComLocalPose.GetLocation());
	Body_External.SetRotationOfMass(InComLocalPose.GetRotation());
}

bool FChaosEngineInterface::IsInertiaConditioningEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference)
{
	return InActorReference->GetGameThreadAPI().InertiaConditioningEnabled();
}
void FChaosEngineInterface::SetInertiaConditioningEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bEnabled)
{
	InActorReference->GetGameThreadAPI().SetInertiaConditioningEnabled(bEnabled);
}

void FChaosEngineInterface::SetIsSimulationShape(const FPhysicsShapeHandle& InShape,bool bIsSimShape)
{
	InShape.Shape->SetSimEnabled(bIsSimShape);
}

void FChaosEngineInterface::SetIsProbeShape(const FPhysicsShapeHandle& InShape, bool bIsProbeShape)
{
	InShape.Shape->SetIsProbe(bIsProbeShape);
}

void FChaosEngineInterface::SetIsQueryShape(const FPhysicsShapeHandle& InShape,bool bIsQueryShape)
{
	InShape.Shape->SetQueryEnabled(bIsQueryShape);
}

float FChaosEngineInterface::GetStabilizationEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InHandle)
{
	// #todo : Implement
	return 0.0f;
}

void FChaosEngineInterface::SetStabilizationEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InHandle,float InThreshold)
{
	// #todo : Implement
}

void FChaosEngineInterface::SetSolverPositionIterationCount_AssumesLocked(const FPhysicsActorHandle& InHandle, int32 InSolverIterationCount)
{
	Chaos::FRigidBodyHandle_External& Body_External = InHandle->GetGameThreadAPI();
	Body_External.SetPositionSolverIterationCount(InSolverIterationCount);
}

void FChaosEngineInterface::SetSolverVelocityIterationCount_AssumesLocked(const FPhysicsActorHandle& InHandle, int32 InSolverIterationCount)
{
	Chaos::FRigidBodyHandle_External& Body_External = InHandle->GetGameThreadAPI();
	Body_External.SetVelocitySolverIterationCount(InSolverIterationCount);
}

void FChaosEngineInterface::SetSolverProjectionIterationCount_AssumesLocked(const FPhysicsActorHandle& InHandle, int32 InSolverIterationCount)
{
	Chaos::FRigidBodyHandle_External& Body_External = InHandle->GetGameThreadAPI();
	Body_External.SetProjectionSolverIterationCount(InSolverIterationCount);
}


float FChaosEngineInterface::GetWakeCounter_AssumesLocked(const FPhysicsActorHandle& InHandle)
{
	// #todo : Implement
	return 0.0f;
}

void FChaosEngineInterface::SetWakeCounter_AssumesLocked(const FPhysicsActorHandle& InHandle,float InWakeCounter)
{
	// #todo : Implement
}

void FChaosEngineInterface::SetInitialized_AssumesLocked(const FPhysicsActorHandle& InHandle,bool InInitialized)
{
	//why is this needed?
	Chaos::FPBDRigidParticle* Rigid = InHandle->GetParticle_LowLevel()->CastToRigidParticle();
	if(Rigid)
	{
		Rigid->SetInitialized(InInitialized);
	}
}

SIZE_T FChaosEngineInterface::GetResourceSizeEx(const FPhysicsActorHandle& InActorRef)
{
	return sizeof(FPhysicsActorHandle);
}

// Constraints
FPhysicsConstraintHandle FChaosEngineInterface::CreateConstraint(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2, const FTransform& InLocalFrame1, const FTransform& InLocalFrame2)
{
	FPhysicsConstraintHandle ConstraintRef;

	if (bEnableChaosJointConstraints)
	{
		Chaos::FPhysicsSolver* Solver1 = Chaos::FPhysicsObjectInterface::GetSolver({ &Body1, 1 });
		Chaos::FPhysicsSolver* Solver2 = Chaos::FPhysicsObjectInterface::GetSolver({ &Body2, 1 });

		if (Body1 && Body2 && Solver1 && Solver2)
		{
			LLM_SCOPE(ELLMTag::ChaosConstraint);

			auto* JointConstraint = new Chaos::FJointConstraint();
			ConstraintRef.Constraint = JointConstraint;

			JointConstraint->SetPhysicsBodies({ Body1, Body2 });
			JointConstraint->SetJointTransforms({ InLocalFrame1,InLocalFrame2 });

			checkSlow(Solver1 == Solver2);
			Solver1->RegisterObject(JointConstraint);
		}
		else if (Body1 || Body2)
		{
			LLM_SCOPE(ELLMTag::ChaosConstraint);

			Chaos::FPhysicsObject* ValidObject = Body1;
			Chaos::FPhysicsSolver* ValidSolver = Solver1;
			bool bSwapped = false;
			if (!ValidObject || !ValidSolver)
			{
				bSwapped = true;
				ValidObject = Body2;
				ValidSolver = Solver2;
			}

			if (ValidSolver)
			{
				FChaosScene* Scene = PhysicsObjectPhysicsCoreInterface::GetScene({ &ValidObject, 1 });

				// Create kinematic actor to attach to joint
				FPhysicsActorHandle KinematicEndPoint;
				FActorCreationParams Params;
				Params.bSimulatePhysics = false;
				Params.bQueryOnly = false;
				Params.Scene = Scene;
				Params.bStatic = false;
				Params.InitialTM = FTransform::Identity;

#if CHAOS_DEBUG_NAME
				TAnsiStringBuilder<256> EndPointDebugNameBuilder;
				EndPointDebugNameBuilder.Appendf("Constraint Kinematic Endpoint - [%s]", TCHAR_TO_ANSI(*Chaos::FPhysicsObjectInterface::GetDebugName(ValidObject)));

				Params.DebugName = EndPointDebugNameBuilder.GetData();
#endif

				FChaosEngineInterface::CreateActor(Params, KinematicEndPoint);

				// Chaos requires our particles have geometry.
				auto Sphere = MakeImplicitObjectPtr<Chaos::FImplicitSphere3>(FVector(0, 0, 0), 0);
				KinematicEndPoint->GetGameThreadAPI().SetGeometry(Sphere);
				KinematicEndPoint->GetGameThreadAPI().SetUserData(nullptr);

				auto* JointConstraint = new Chaos::FJointConstraint();
				JointConstraint->SetKinematicEndPoint(KinematicEndPoint, Scene->GetSolver());
				ConstraintRef.Constraint = JointConstraint;

				// Disable collision on shape to ensure it is not added to acceleration structure.
				for (const TUniquePtr<Chaos::FPerShapeData>& Shape : KinematicEndPoint->GetGameThreadAPI().ShapesArray())
				{
					Chaos::FCollisionData CollisionData = Shape->GetCollisionData();
					CollisionData.bQueryCollision = false;
					CollisionData.bSimCollision = false;
					Shape->SetCollisionData(CollisionData);
				}

				JointConstraint->SetPhysicsBodies({ ValidObject, KinematicEndPoint->GetPhysicsObject() });

				Chaos::FTransformPair TransformPair = { InLocalFrame1, InLocalFrame2 };
				if (bSwapped)
				{
					Swap(TransformPair[0], TransformPair[1]);
				}
				JointConstraint->SetJointTransforms(TransformPair);

				checkSlow(ValidSolver == KinematicEndPoint->GetSolver<Chaos::FPhysicsSolver>());
				ValidSolver->RegisterObject(JointConstraint);
			}
		}
	}

	return ConstraintRef;
}

FPhysicsConstraintHandle FChaosEngineInterface::CreateConstraint(const FPhysicsActorHandle& InActorRef1,const FPhysicsActorHandle& InActorRef2,const FTransform& InLocalFrame1,const FTransform& InLocalFrame2)
{
	Chaos::FPhysicsObject* Body1 = InActorRef1 ? InActorRef1->GetPhysicsObject() : nullptr;
	Chaos::FPhysicsObject* Body2 = InActorRef2 ? InActorRef2->GetPhysicsObject() : nullptr;
	return FChaosEngineInterface::CreateConstraint(Body1, Body2, InLocalFrame1, InLocalFrame2);
}

FPhysicsConstraintHandle FChaosEngineInterface::CreateSuspension(const FPhysicsActorHandle& InActorRef, const FVector& InLocalFrame)
{
	Chaos::FPhysicsObject* Body = InActorRef ? InActorRef->GetPhysicsObject() : nullptr;
	return FChaosEngineInterface::CreateSuspension(Body, InLocalFrame);
}

FPhysicsConstraintHandle FChaosEngineInterface::CreateSuspension(Chaos::FPhysicsObject* Body, const FVector& InLocalFrame)
{
	FPhysicsConstraintHandle ConstraintRef;

	if (bEnableChaosJointConstraints)
	{
		if (Body)
		{
			Chaos::FPhysicsSolver* Solver = Chaos::FPhysicsObjectInterface::GetSolver({ &Body, 1 });

			if (Solver)
			{
				LLM_SCOPE(ELLMTag::ChaosConstraint);

				auto* SuspensionConstraint = new Chaos::FSuspensionConstraint();
				ConstraintRef.Constraint = SuspensionConstraint;

				SuspensionConstraint->SetPhysicsBody(Body);
				SuspensionConstraint->SetLocation(InLocalFrame);

				Solver->RegisterObject(SuspensionConstraint);
			}
		}
	}
	return ConstraintRef;
}


void FChaosEngineInterface::SetConstraintUserData(const FPhysicsConstraintHandle& InConstraintRef,void* InUserData)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetUserData(InUserData);
		}
	}
}

void FChaosEngineInterface::ReleaseConstraint(FPhysicsConstraintHandle& InConstraintRef)
{
	using namespace Chaos;
	if (bEnableChaosJointConstraints)
	{
		if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
		{
			if (FJointConstraint* Constraint = static_cast<FJointConstraint*>(InConstraintRef.Constraint))
			{
				if (FJointConstraintPhysicsProxy* Proxy = Constraint->GetProxy<FJointConstraintPhysicsProxy>())
				{
					check(Proxy->GetSolver<FPhysicsSolver>());
					FPhysicsSolver* Solver = Proxy->GetSolver<FPhysicsSolver>();
					// TODO: we should probably figure out a way to call this from within UnregisterObject, to match
					// what RegisterObject does
					if (FChaosScene* Scene = FChaosEngineInterface::GetCurrentScene(Constraint->GetKinematicEndPoint()))
					{
						Scene->RemoveActorFromAccelerationStructure(Constraint->GetKinematicEndPoint());
					}
					Solver->UnregisterObject(Constraint);

					InConstraintRef.Constraint = nullptr; // freed by the joint constraint physics proxy
				}
			}
		}
		else if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(EConstraintType::SuspensionConstraintType))
		{
			if (Chaos::FSuspensionConstraint* Constraint = static_cast<FSuspensionConstraint*>(InConstraintRef.Constraint))
			{
				if (FSuspensionConstraintPhysicsProxy* Proxy = Constraint->GetProxy<FSuspensionConstraintPhysicsProxy>())
				{
					check(Proxy->GetSolver<FPhysicsSolver>());
					FPhysicsSolver* Solver = Proxy->GetSolver<FPhysicsSolver>();

					Solver->UnregisterObject(Constraint);

					InConstraintRef.Constraint = nullptr;  // freed by the joint constraint physics proxy
				}
			}

		}
	}
}

FTransform FChaosEngineInterface::GetLocalPose(const FPhysicsConstraintHandle& InConstraintRef,EConstraintFrame::Type InFrame)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			const Chaos::FTransformPair& M = Constraint->GetJointTransforms();
			if (InFrame == EConstraintFrame::Frame1)
			{
				return M[0];
			}
			else if (InFrame == EConstraintFrame::Frame2)
			{
				return M[1];
			}
		}
	}
	return FTransform::Identity;
}

Chaos::FGeometryParticle*
GetParticleFromProxy(IPhysicsProxyBase* ProxyBase)
{
	if (ProxyBase)
	{
		if (ProxyBase->GetType() == EPhysicsProxyType::SingleParticleProxy)
		{
			return ((Chaos::FSingleParticlePhysicsProxy*)ProxyBase)->GetParticle_LowLevel();
		}
	}
	return nullptr;
}


FTransform FChaosEngineInterface::GetGlobalPose(const FPhysicsConstraintHandle& InConstraintRef, EConstraintFrame::Type InFrame)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Chaos::FProxyBasePair BasePairs = Constraint->GetParticleProxies();
			const Chaos::FTransformPair& M = Constraint->GetJointTransforms();

			if (InFrame == EConstraintFrame::Frame1)
			{
				if (Chaos::FGeometryParticle* Particle = GetParticleFromProxy(BasePairs[0]))
				{
					return M[0] * FTransform(Particle->R(), Particle->X());
				}
			}
			else if (InFrame == EConstraintFrame::Frame2)
			{
				if (Chaos::FGeometryParticle* Particle = GetParticleFromProxy(BasePairs[1]))
				{
					return M[1] * FTransform(Particle->R(), Particle->X());
				}
			}
		}
	}
	return FTransform::Identity;
}

FVector FChaosEngineInterface::GetLocation(const FPhysicsConstraintHandle& InConstraintRef)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			return 0.5 * (GetGlobalPose(InConstraintRef, EConstraintFrame::Frame1).GetTranslation() + GetGlobalPose(InConstraintRef, EConstraintFrame::Frame2).GetTranslation());
		}
	}
	return FVector::ZeroVector;

}

void FChaosEngineInterface::GetForce(const FPhysicsConstraintHandle& InConstraintRef, FVector& OutLinForce, FVector& OutAngForce)
{
	OutLinForce = FVector::ZeroVector;
	OutAngForce = FVector::ZeroVector;

	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			OutLinForce = Constraint->GetOutputData().Force;
			OutAngForce = Constraint->GetOutputData().Torque;
		}
	}
}

void FChaosEngineInterface::GetDriveLinearVelocity(const FPhysicsConstraintHandle& InConstraintRef,FVector& OutLinVelocity)
{
	OutLinVelocity = FVector::ZeroVector;

	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			OutLinVelocity = Constraint->GetLinearDriveVelocityTarget();
		}
	}
}

void FChaosEngineInterface::GetDriveAngularVelocity(const FPhysicsConstraintHandle& InConstraintRef,FVector& OutAngVelocity)
{
	OutAngVelocity = FVector::ZeroVector;

	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			OutAngVelocity = Constraint->GetAngularDriveVelocityTarget();
		}
	}
}

FTransform GetConstraintBodiesRelativeTransform(const FPhysicsConstraintHandle& InConstraintRef)
{
	const FTransform ChildTransform = FChaosEngineInterface::GetGlobalPose(InConstraintRef, EConstraintFrame::Frame1);
	const FTransform ParentTransform = FChaosEngineInterface::GetGlobalPose(InConstraintRef, EConstraintFrame::Frame2);
	return ChildTransform.GetRelativeTransform(ParentTransform);
}

float FChaosEngineInterface::GetCurrentSwing1(const FPhysicsConstraintHandle& InConstraintRef)
{
	return GetConstraintBodiesRelativeTransform(InConstraintRef).GetRotation().GetTwistAngle(Chaos::FJointConstants::Swing1Axis());
}

float FChaosEngineInterface::GetCurrentSwing2(const FPhysicsConstraintHandle& InConstraintRef)
{
	return GetConstraintBodiesRelativeTransform(InConstraintRef).GetRotation().GetTwistAngle(Chaos::FJointConstants::Swing2Axis());
}

float FChaosEngineInterface::GetCurrentTwist(const FPhysicsConstraintHandle& InConstraintRef)
{
	return GetConstraintBodiesRelativeTransform(InConstraintRef).GetRotation().GetTwistAngle(Chaos::FJointConstants::TwistAxis());
}

void FChaosEngineInterface::SetCanVisualize(const FPhysicsConstraintHandle& InConstraintRef,bool bInCanVisualize)
{
	// @todo(chaos) :  Joint Constraints : Debug Tools
}

void FChaosEngineInterface::SetCollisionEnabled(const FPhysicsConstraintHandle& InConstraintRef,bool bInCollisionEnabled)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetCollisionEnabled(bInCollisionEnabled);
		}
	}
}

void FChaosEngineInterface::SetProjectionEnabled_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef,bool bInProjectionEnabled,float InLinearAlpha,float InAngularAlpha, float InLinearTolerance, float InAngularToleranceDeg)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetProjectionEnabled(bInProjectionEnabled);
			Constraint->SetProjectionLinearAlpha(InLinearAlpha);
			Constraint->SetProjectionAngularAlpha(InAngularAlpha);
			Constraint->SetProjectionLinearTolerance(InLinearTolerance);
			Constraint->SetProjectionAngularTolerance(FMath::DegreesToRadians(InAngularToleranceDeg));
		}
	}
}

void FChaosEngineInterface::SetShockPropagationEnabled_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, bool bInShockPropagationEnabled, float InShockPropagationAlpha)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetShockPropagationEnabled(bInShockPropagationEnabled);
			Constraint->SetShockPropagationAlpha(InShockPropagationAlpha);
		}
	}
}

void FChaosEngineInterface::SetParentDominates_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef,bool bInParentDominates)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			if(bInParentDominates)
			{
				Constraint->SetParentInvMassScale(0.f);
			} else
			{
				Constraint->SetParentInvMassScale(1.f);
			}
		}
	}
}

void FChaosEngineInterface::SetMassConditioningEnabled_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, bool bInMassConditioningEnabled)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetMassConditioningEnabled(bInMassConditioningEnabled);
		}
	}
}

void FChaosEngineInterface::SetUseLinearJointSolver_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, bool bInUseLinearJointSolver)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetUseLinearSolver(bInUseLinearJointSolver);
		}
	}
}

void FChaosEngineInterface::SetBreakForces_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef,float InLinearBreakForce,float InAngularBreakTorque)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearBreakForce(InLinearBreakForce);
			Constraint->SetAngularBreakTorque(InAngularBreakTorque);
		}
	}
}

void FChaosEngineInterface::SetViolationCallbackThresholds_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef,float InLinearViolationCallbackThreshold,float InAngularViolationCallbackThreshold)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearViolationCallbackThreshold(InLinearViolationCallbackThreshold);
			Constraint->SetAngularViolationCallbackThreshold(InAngularViolationCallbackThreshold);
		}
	}
}

void FChaosEngineInterface::SetPlasticityLimits_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InLinearPlasticityLimit, float InAngularPlasticityLimit, EConstraintPlasticityType InLinearPlasticityType)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearPlasticityType((Chaos::EPlasticityType)InLinearPlasticityType);
			Constraint->SetLinearPlasticityLimit(InLinearPlasticityLimit);
			Constraint->SetAngularPlasticityLimit(InAngularPlasticityLimit);
		}
	}
}

void FChaosEngineInterface::SetContactTransferScale_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InContactTransferScale)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetContactTransferScale(InContactTransferScale);
		}
	}
}

void FChaosEngineInterface::SetLocalPose(const FPhysicsConstraintHandle& InConstraintRef,const FTransform& InPose,EConstraintFrame::Type InFrame)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Chaos::FTransformPair JointTransforms = Constraint->GetJointTransforms();
			if (InFrame == EConstraintFrame::Frame1)
			{
				JointTransforms[0] = InPose;
			}
			else
			{
				JointTransforms[1] = InPose;
			}
			Constraint->SetJointTransforms(JointTransforms);
		}
	}
}

void FChaosEngineInterface::SetDrivePosition(const FPhysicsConstraintHandle& InConstraintRef,const FVector& InPosition)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearDrivePositionTarget(InPosition);
		}
	}
}

void FChaosEngineInterface::SetDriveOrientation(const FPhysicsConstraintHandle& InConstraintRef,const FQuat& InOrientation)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetAngularDrivePositionTarget(InOrientation);
		}
	}
}

void FChaosEngineInterface::SetDriveLinearVelocity(const FPhysicsConstraintHandle& InConstraintRef,const FVector& InLinVelocity)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearDriveVelocityTarget(InLinVelocity);
		}
	}
}

void FChaosEngineInterface::SetDriveAngularVelocity(const FPhysicsConstraintHandle& InConstraintRef,const FVector& InAngVelocity)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetAngularDriveVelocityTarget(InAngVelocity);
		}
	}
}

void FChaosEngineInterface::SetTwistLimit(const FPhysicsConstraintHandle& InConstraintRef,float InLowerLimit,float InUpperLimit,float InContactDistance)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Chaos::FVec3 Limit = Constraint->GetAngularLimits();
			Limit[(int32)Chaos::EJointAngularConstraintIndex::Twist] = FMath::DegreesToRadians(InUpperLimit - InLowerLimit);
			Constraint->SetAngularLimits(Limit);
			Constraint->SetTwistContactDistance(InContactDistance);
		}
	}
}

void FChaosEngineInterface::SetSwingLimit(const FPhysicsConstraintHandle& InConstraintRef,float InYLimit,float InZLimit,float InContactDistance)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Chaos::FVec3 Limit = Constraint->GetAngularLimits();
			Limit[(int32)Chaos::EJointAngularConstraintIndex::Swing1] = FMath::DegreesToRadians(InYLimit);
			Limit[(int32)Chaos::EJointAngularConstraintIndex::Swing2] = FMath::DegreesToRadians(InZLimit);
			Constraint->SetAngularLimits(Limit);
			Constraint->SetSwingContactDistance(InContactDistance);
		}
	}
}

void FChaosEngineInterface::SetLinearLimit(const FPhysicsConstraintHandle& InConstraintRef,float InLinearLimit)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			Constraint->SetLinearLimit(InLinearLimit);
		}
	}
}

bool FChaosEngineInterface::IsBroken(const FPhysicsConstraintHandle& InConstraintRef)
{
	if (InConstraintRef.IsValid() && InConstraintRef.Constraint->IsType(Chaos::EConstraintType::JointConstraintType))
	{
		if (Chaos::FJointConstraint* Constraint = static_cast<Chaos::FJointConstraint*>(InConstraintRef.Constraint))
		{
			return Constraint->GetOutputData().bIsBroken;
		}
	}
	return false;
}


void FChaosEngineInterface::SetGeometry(FPhysicsShapeHandle& InShape, Chaos::FImplicitObjectPtr&& InGeometry)
{
	using namespace Chaos;

	// This sucks, we build a new union with input geometry. All other geo is copied.
	// Cannot modify union as it is shared between threads.
	const FShapesArray& ShapeArray = InShape.ActorRef->GetGameThreadAPI().ShapesArray();

	TArray<Chaos::FImplicitObjectPtr> NewGeometry;
	NewGeometry.Reserve(ShapeArray.Num());

	int32 ShapeIdx = 0;
	for (const TUniquePtr<Chaos::FPerShapeData>& Shape : ShapeArray)
	{
		if (Shape.Get() == InShape.Shape)
		{
			NewGeometry.Emplace(MoveTemp(InGeometry));
		}
		else
		{
			NewGeometry.Emplace(Shape->GetGeometry()->CopyGeometry());
		}

		ShapeIdx++;
	}

	if (ensure(NewGeometry.Num() == ShapeArray.Num()))
	{
		Chaos::FImplicitObjectPtr ImplicitUnion = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(NewGeometry));
		InShape.ActorRef->GetGameThreadAPI().SetGeometry(ImplicitUnion);
		
		FChaosScene* Scene = FChaosEngineInterface::GetCurrentScene(InShape.ActorRef);
		if (ensure(Scene))
		{
			Scene->UpdateActorInAccelerationStructure(InShape.ActorRef);
		}
	}
}

// @todo(chaos): We probably need to actually duplicate the data here, add virtual TImplicitObject::NewCopy()
FPhysicsShapeHandle FChaosEngineInterface::CloneShape(const FPhysicsShapeHandle& InShape)
{
	FPhysicsActorHandle NewActor = nullptr;
	return {InShape.Shape,NewActor};
}

FPhysicsGeometryCollection_Chaos FChaosEngineInterface::GetGeometryCollection(const FPhysicsShapeHandle& InShape)
{
	FPhysicsGeometryCollection_Chaos NewCollection(InShape);
	return NewCollection;
}

FPhysicsGeometryCollection_Chaos FChaosEngineInterface::GetGeometryCollection(const FPhysicsGeometry& InShape)
{
	return FPhysicsGeometryCollection_Chaos{ InShape };
}

void FChaosEngineInterface::SetMaskFilter(const FPhysicsShapeHandle& InShape, FMaskFilter InFilter)
{
	Chaos::Filter::FShapeFilterData FilterData = GetShapeFilterData(InShape);
	FilterData.SetMaskFilter(InFilter);
	SetShapeFilterData(InShape, FilterData);
}

FCollisionFilterData FChaosEngineInterface::GetSimulationFilter(const FPhysicsShapeReference_Chaos& InShape)
{
	if(ensure(InShape.Shape))
	{
		return Chaos::Filter::FShapeFilterBuilder::GetLegacyShapeSimFilter(InShape.Shape->GetCombinedShapeFilterData());
	} else
	{
		return FCollisionFilterData();
	}
}

FCollisionFilterData FChaosEngineInterface::GetQueryFilter(const FPhysicsShapeReference_Chaos& InShape)
{
	if(ensure(InShape.Shape))
	{
		return Chaos::Filter::FShapeFilterBuilder::GetLegacyShapeQueryFilter(InShape.Shape->GetCombinedShapeFilterData());
	} else
	{
		return FCollisionFilterData();
	}
}

Chaos::Filter::FShapeFilterData FChaosEngineInterface::GetShapeFilterData(const FPhysicsShapeHandle& InShape)
{
	if (ensure(InShape.Shape))
	{
		return InShape.Shape->GetShapeFilterData();
	}
	else
	{
		return Chaos::Filter::FShapeFilterData();
	}
}

Chaos::Filter::FInstanceData FChaosEngineInterface::GetFilterInstanceData(const FPhysicsShapeHandle& InShape)
{
	if (ensure(InShape.Shape))
	{
		return InShape.Shape->GetFilterInstanceData();
	}
	else
	{
		return Chaos::Filter::FInstanceData();
	}
}

Chaos::Filter::FCombinedShapeFilterData FChaosEngineInterface::GetCombinedShapeFilterData(const FPhysicsShapeHandle& InShape)
{
	if (ensure(InShape.Shape))
	{
		return InShape.Shape->GetCombinedShapeFilterData();
	}
	else
	{
		return Chaos::Filter::FCombinedShapeFilterData();
	}
}

void FChaosEngineInterface::SetQueryFilter(const FPhysicsShapeReference_Chaos& InShapeRef,const FCollisionFilterData& InFilter)
{
	Chaos::Filter::FCombinedShapeFilterData CombinedShapeFilter = GetCombinedShapeFilterData(InShapeRef);
	Chaos::Filter::FShapeFilterBuilder::SetLegacyShapeQueryFilter(CombinedShapeFilter, InFilter);
	SetCombinedShapeFilterData(InShapeRef, CombinedShapeFilter);
}

void FChaosEngineInterface::SetSimulationFilter(const FPhysicsShapeReference_Chaos& InShapeRef,const FCollisionFilterData& InFilter)
{
	Chaos::Filter::FCombinedShapeFilterData CombinedShapeFilter = GetCombinedShapeFilterData(InShapeRef);
	Chaos::Filter::FShapeFilterBuilder::SetLegacyShapeSimFilter(CombinedShapeFilter, InFilter);
	SetCombinedShapeFilterData(InShapeRef, CombinedShapeFilter);
}

void FChaosEngineInterface::SetShapeFilterData(const FPhysicsShapeHandle& InShapeRef, const Chaos::Filter::FShapeFilterData& InFilter)
{
	InShapeRef.Shape->SetShapeFilterData(InFilter);
}

void FChaosEngineInterface::SetFilterInstanceData(const FPhysicsShapeHandle& InShapeRef, const Chaos::Filter::FInstanceData& InFilter)
{
	InShapeRef.Shape->SetFilterInstanceData(InFilter);
}

void FChaosEngineInterface::SetCombinedShapeFilterData(const FPhysicsShapeReference_Chaos& InShapeRef, const Chaos::Filter::FCombinedShapeFilterData& InFilter)
{
	InShapeRef.Shape->SetCombinedShapeFilterData(InFilter);
}

bool FChaosEngineInterface::IsSimulationShape(const FPhysicsShapeHandle& InShape)
{
	return InShape.Shape->GetSimEnabled();
}

bool FChaosEngineInterface::IsQueryShape(const FPhysicsShapeHandle& InShape)
{
	// This data is not stored on concrete shape. TODO: Remove ensure if we actually use this flag when constructing shape handles.
	CHAOS_ENSURE(false);
	return InShape.Shape->GetQueryEnabled();
}

ECollisionShapeType FChaosEngineInterface::GetShapeType(const FPhysicsShapeReference_Chaos& InShapeRef)
{
	return ChaosInterface::GetImplicitType(*InShapeRef.Shape->GetGeometry());
}

FTransform FChaosEngineInterface::GetLocalTransform(const FPhysicsShapeReference_Chaos& InShapeRef)
{
	// Transforms are baked into the object so there is never a local transform
	if(InShapeRef.Shape->GetGeometry()->GetType() == Chaos::ImplicitObjectType::Transformed && FChaosEngineInterface::IsValid(InShapeRef.ActorRef))
	{
		return InShapeRef.Shape->GetGeometry()->GetObject<Chaos::TImplicitObjectTransformed<Chaos::FReal,3>>()->GetTransform();
	} else
	{
		return FTransform();
	}
}

void FChaosEngineInterface::SetLocalTransform(const FPhysicsShapeHandle& InShape,const FTransform& NewLocalTransform)
{
	using namespace Chaos;

	FSingleParticlePhysicsProxy* Particle = InShape.ActorRef;
	if(Particle)
	{
		Chaos::FRigidBodyHandle_External& BodyHandle = Particle->GetGameThreadAPI();

		const FImplicitObjectRef CurrentGeom = BodyHandle.GetGeometry();
		if(ensure(CurrentGeom && CurrentGeom->GetType() == FImplicitObjectUnion::StaticType()))
		{
			const FImplicitObjectUnion* AsUnion = static_cast<const FImplicitObjectUnion*>(CurrentGeom);
			const int32 ShapeIndex = InShape.Shape->GetShapeIndex();
			const TArray<Chaos::FImplicitObjectPtr>& ObjectArray = AsUnion->GetObjects();

			if(ensure(ShapeIndex < ObjectArray.Num()))
			{
				TArray<Chaos::FImplicitObjectPtr> NewGeoms;
				NewGeoms.Reserve(ObjectArray.Num());

				// Duplicate the union and either set transforms, or wrap in transforms
				int32 CurrentIndex = 0;
				for(const Chaos::FImplicitObjectPtr& Obj : ObjectArray)
				{
					if(CurrentIndex == ShapeIndex)
					{
						NewGeoms.Emplace(Utilities::DuplicateGeometryWithTransform(Obj.GetReference(), NewLocalTransform));
					}
					else
					{
						NewGeoms.Emplace(Obj->CopyGeometry());
					}

					CurrentIndex++;
				}

				if(ensure(NewGeoms.Num() == ObjectArray.Num()))
				{
					Chaos::FImplicitObjectPtr ImplicitUnion = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(NewGeoms));
					BodyHandle.SetGeometry(ImplicitUnion);
				}
			}
		}
	}
}

void FChaosEngineInterface::SerializeParticleStateToBuffer(const FPhysicsActorHandle& InActorHandle, Chaos::FSerializedDataBuffer& OutSerializedData, Chaos::ESerializedDataContext DataContext)
{
	if (!IsValid(InActorHandle))
	{
		UE_LOGF(LogChaos, Warning, "[%s]: Failed to serialize particle | Invalid Actor Handle.", __func__);
		return;
	}

	using namespace Chaos;

	auto SerializeInternalStateToBuffer = [InActorHandle](FPBDRigidsSolver& Solver, FSerializedDataBuffer& OutData)
	{
		if (InActorHandle->GetHandle_LowLevel())
		{
			Solver.GetSerializer().SerializeParticleStateToBuffer(InActorHandle->GetHandle_LowLevel(), OutData);
		}
		else
		{
			if (FSerializedDataBufferPtr PendingDataHandle = Solver.GetSerializer().PopPendingInternalSerializedStateForProxy(InActorHandle))
			{
				OutData.GetDataAsByteArrayRef().Append(PendingDataHandle->GetDataAsByteArrayRef());
			}
			else
			{
				SerializeParticleStateToBuffer(InActorHandle, OutData, ESerializedDataContext::External);
				UE_LOGF(LogChaos, Warning, "[%s]: Failed to get a valid Internal physics state. Falling back to the external state | Owner [%ls].", __func__, *GetNameSafe(InActorHandle->GetOwner()));
			}
		}
	};

	if (FPBDRigidsSolver* Solver = InActorHandle->GetSolver<FPBDRigidsSolver>())
	{
		switch (DataContext)
		{
			case ESerializedDataContext::External:
				Solver->GetSerializer().SerializeParticleStateToBuffer(InActorHandle->GetParticle_LowLevel(), OutSerializedData);
				break;
			case ESerializedDataContext::Internal:
				SerializeInternalStateToBuffer(*Solver, OutSerializedData);
				break;
			case ESerializedDataContext::Invalid:
			case ESerializedDataContext::Both:
			default:
				{
					// Note: We don't support serialize data from both states into a single buffer yet. But we do support apply the same state to External and Internal during deserialization if ESerializedDataContext::Both is used.
					UE_LOGF(LogChaos, Warning, "[%s]: Failed to serialize particle | Unsupported Data Context [%ls].", __func__, Chaos::LexToString(DataContext));
					break;
				}
		}
	}
}

void FChaosEngineInterface::ApplySerializedStateToParticle(const FPhysicsActorHandle& InActorHandle, Chaos::FSerializedDataBufferPtr&& InSerializedData, Chaos::ESerializedDataContext DataContext)
{
	if (!IsValid(InActorHandle) || !InSerializedData)
	{
		return;
	}

	using namespace Chaos;

	auto ApplySerializedPTSerializedState = [&InActorHandle](FPBDRigidsSolver& Solver, Chaos::FSerializedDataBufferPtr&& InSerializedData)
	{
		if (FGeometryParticleHandle* ParticleHandle = InActorHandle->GetHandle_LowLevel())
		{
			Solver.GetSerializer().ApplySerializedStateToParticle(ParticleHandle, *InSerializedData);
		}
		else
		{
			Solver.GetSerializer().PushPendingInternalSerializedStateForProxy(InActorHandle, MoveTemp(InSerializedData));
		}
	};

	if (FPBDRigidsSolver* Solver = InActorHandle->GetSolver<FPBDRigidsSolver>())
	{
		switch (DataContext)
		{
			case ESerializedDataContext::External:
				Solver->GetSerializer().ApplySerializedStateToParticle(InActorHandle->GetParticle_LowLevel(), *InSerializedData);
				break;
			case ESerializedDataContext::Internal:
				{
					ApplySerializedPTSerializedState(*Solver, MoveTemp(InSerializedData));
					break;
				}
			case ESerializedDataContext::Both:
				{
					Solver->GetSerializer().ApplySerializedStateToParticle(InActorHandle->GetParticle_LowLevel(), *InSerializedData);
					ApplySerializedPTSerializedState(*Solver, MoveTemp(InSerializedData));
					break;
				}
		case ESerializedDataContext::Invalid:
		default:
			UE_LOGF(LogChaos, Warning, "[%s]: Failed to apply serialized state to particle | Unsupported Data Context [%ls].", __func__, Chaos::LexToString(DataContext));
			break;
		}
	}
}

void FChaosEngineInterface::SerializeConstraintStateToBuffer(const FPhysicsConstraintHandle& ConstraintHandle, Chaos::FSerializedDataBuffer& OutSerializedData, Chaos::ESerializedDataContext DataContext)
{
	if (!ConstraintHandle.IsValid())
	{
		UE_LOGF(LogChaos, Warning, "[%s]: Failed to constraint particle | Invalid Actor Handle.", __func__);
		return;
	}

	using namespace Chaos;

	IPhysicsProxyBase* Proxy = ConstraintHandle.Constraint->GetProxy();

	if (FPBDRigidsSolver* Solver = Proxy ? Proxy->GetSolver<FPBDRigidsSolver>() : nullptr)
	{
		const bool bIsConstraintProxy = Proxy->GetType() == EPhysicsProxyType::CharacterGroundConstraintType ||
										Proxy->GetType() == EPhysicsProxyType::JointConstraintType ||
										Proxy->GetType() == EPhysicsProxyType::SuspensionConstraintType;
		if (!ensure(bIsConstraintProxy))
		{
			return;
		}

		auto SerializeInternalStateToBuffer = [&ConstraintHandle](IPhysicsProxyBase* Proxy, FPBDRigidsSolver& Solver, FSerializedDataBuffer& OutData)
		{
			if (FConstraintHandle* PTConstraintHandle = static_cast<FConstraintHandle*>(Proxy->GetHandleUnsafe()))
			{
				Solver.GetSerializer().SerializeConstraintStateToBuffer(PTConstraintHandle, OutData);
			}
			else
			{
				if (FSerializedDataBufferPtr PendingDataHandle = Solver.GetSerializer().PopPendingInternalSerializedStateForProxy(Proxy))
				{
					OutData.GetDataAsByteArrayRef().Append(PendingDataHandle->GetDataAsByteArrayRef());
				}
				else
				{
					SerializeConstraintStateToBuffer(ConstraintHandle, OutData, ESerializedDataContext::External);
					UE_LOGF(LogChaos, Warning, "[%s]: Failed to get a valid Internal physics state. Falling back to the external state | Owner [%ls].", __func__, *GetNameSafe(Proxy->GetOwner()));
				}
			}
		};

		switch (DataContext)
		{
		case ESerializedDataContext::External:
			Solver->GetSerializer().SerializeConstraintStateToBuffer(ConstraintHandle.Constraint, OutSerializedData);
			break;
		case ESerializedDataContext::Internal:
			SerializeInternalStateToBuffer(Proxy, *Solver, OutSerializedData);
			break;
		case ESerializedDataContext::Invalid:
		case ESerializedDataContext::Both:
		default:
			{
				// Note: We don't support serialize data from both states into a single buffer yet. But we do support apply the same state to External and Internal during deserialization if ESerializedDataContext::Both is used.
				UE_LOGF(LogChaos, Warning, "[%s]: Failed to serialize constraint | Unsupported Data Context [%ls].", __func__, Chaos::LexToString(DataContext));
				break;
			}
		}
	}
}

void FChaosEngineInterface::ApplySerializedStateToConstraint(const FPhysicsConstraintHandle& ConstraintHandle, Chaos::FSerializedDataBufferPtr&& InSerializedData, Chaos::ESerializedDataContext DataContext)
{
	if (!ConstraintHandle.IsValid() || !InSerializedData)
	{
		return;
	}

	using namespace Chaos;

	IPhysicsProxyBase* Proxy = ConstraintHandle.Constraint->GetProxy();

	auto ApplySerializedPTSerializedState = [](IPhysicsProxyBase* Proxy, FPBDRigidsSolver& Solver, Chaos::FSerializedDataBufferPtr&& InSerializedData)
	{
		if (FConstraintHandle* PTConstraintHandle = static_cast<FConstraintHandle*>(Proxy->GetHandleUnsafe()))
		{
			Solver.GetSerializer().ApplySerializedStateToConstraint(PTConstraintHandle, *InSerializedData);
		}
		else
		{
			Solver.GetSerializer().PushPendingInternalSerializedStateForProxy(Proxy, MoveTemp(InSerializedData));
		}
	};

	if (FPBDRigidsSolver* Solver = Proxy ? Proxy->GetSolver<FPBDRigidsSolver>() : nullptr)
	{
		switch (DataContext)
		{
			case ESerializedDataContext::External:
				Solver->GetSerializer().ApplySerializedStateToConstraint(ConstraintHandle.Constraint, *InSerializedData);
				break;
			case ESerializedDataContext::Internal:
				{
					ApplySerializedPTSerializedState(Proxy, *Solver, MoveTemp(InSerializedData));
					break;
				}
			case ESerializedDataContext::Both:
				{
					Solver->GetSerializer().ApplySerializedStateToConstraint(ConstraintHandle.Constraint, *InSerializedData);
					ApplySerializedPTSerializedState(Proxy, *Solver, MoveTemp(InSerializedData));
					break;
				}
			case ESerializedDataContext::Invalid:
			default:
				UE_LOGF(LogChaos, Warning, "[%s]: Failed to apply serialized state to particle | Unsupported Data Context [%ls].", __func__, Chaos::LexToString(DataContext));
				break;	
		}
	}
}

template<typename AllocatorType>
int32 GetAllShapesInternalImp_AssumedLocked(const FPhysicsActorHandle& InActorHandle,TArray<FPhysicsShapeReference_Chaos,AllocatorType>& OutShapes)
{
	if(InActorHandle)
	{
		const Chaos::FShapesArray& ShapesArray = InActorHandle->GetGameThreadAPI().ShapesArray();
		const int32 NumRelevantShapes = ShapesArray.Num();
		OutShapes.Reset(NumRelevantShapes);
		
		//todo: can we avoid this construction?
		for(int32 ShapeIndex = 0; ShapeIndex < NumRelevantShapes; ++ShapeIndex)
		{
			OutShapes.Add(FPhysicsShapeReference_Chaos(ShapesArray[ShapeIndex].Get(),InActorHandle));
		}

		return OutShapes.Num();
	}

	return 0;
}

int32 FChaosEngineInterface::GetAllShapes_AssumedLocked(const FPhysicsActorHandle& InActorHandle,TArray<FPhysicsShapeReference_Chaos,FDefaultAllocator>& OutShapes)
{
	return GetAllShapesInternalImp_AssumedLocked(InActorHandle,OutShapes);
}

int32 FChaosEngineInterface::GetAllShapes_AssumedLocked(const FPhysicsActorHandle& InActorHandle,PhysicsInterfaceTypes::FInlineShapeArray& OutShapes)
{
	return GetAllShapesInternalImp_AssumedLocked(InActorHandle,OutShapes);
}

void FChaosEngineInterface::CreateActor(const FActorCreationParams& InParams,FPhysicsActorHandle& Handle, UObject* InOwner)
{
	LLM_SCOPE(ELLMTag::ChaosActor);
	using namespace Chaos;

	TUniquePtr<FGeometryParticle> Particle;
	// Set object state based on the requested particle type
	if(InParams.bStatic)
	{
		Particle = FGeometryParticle::CreateParticle();
	}
	else
	{
		// Create an underlying dynamic particle
		TUniquePtr<FPBDRigidParticle> Rigid = FPBDRigidParticle::CreateParticle();
		Rigid->SetGravityEnabled(InParams.bEnableGravity);
		Rigid->SetUpdateKinematicFromSimulation(InParams.bUpdateKinematicFromSimulation);
		if(InParams.bSimulatePhysics)
		{
			if(InParams.bStartAwake)
			{
				Rigid->SetObjectState(EObjectStateType::Dynamic);
			} else
			{
				Rigid->SetObjectState(EObjectStateType::Sleeping);
			}
		} else
		{
			Rigid->SetObjectState(EObjectStateType::Kinematic);
		}
		//Particle.Reset(Rigid.Release());
		Particle = MoveTemp(Rigid);
	}
	Particle->SetResimType(EResimType::FrozenDuringResim);

	// Set the particle acceleration structure spatial index here
	{
		FSpatialAccelerationIdx SpatialIndex{0, static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::Default) };
		if (AccelerationStructureSplitStaticAndDynamic == 1)
		{
			if (AccelerationStructureIsolateQueryOnlyObjects == 1)
			{
				if (InParams.bStatic && InParams.bQueryOnly)
				{
					SpatialIndex = FSpatialAccelerationIdx{0, static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DefaultQueryOnly)};
				}
				else if (!InParams.bStatic && InParams.bQueryOnly)
				{
					SpatialIndex = FSpatialAccelerationIdx{ 0, static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DynamicQueryOnly)};
				}
				else if (!InParams.bStatic && !InParams.bQueryOnly)
				{
					SpatialIndex = FSpatialAccelerationIdx{ 0, static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::Dynamic) };
				}
			}
			else
			{
				if (!InParams.bStatic)
				{
					SpatialIndex = FSpatialAccelerationIdx{ 0, static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::Dynamic) };
				}				
			}
		}
		else
		{
			if (AccelerationStructureIsolateQueryOnlyObjects == 1)
			{
				if (InParams.bQueryOnly)
				{
					SpatialIndex = FSpatialAccelerationIdx{ 0, static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DefaultQueryOnly) };
				}
			}		
		}
		Particle->SetSpatialIdx(SpatialIndex);
	}

	Handle = Chaos::FSingleParticlePhysicsProxy::Create(MoveTemp(Particle), InOwner);
	Chaos::FRigidBodyHandle_External& Body_External = Handle->GetGameThreadAPI();

	// Set up the new particle's game-thread data. This will be sent to physics-thread when
	// the particle is added to the scene later.
	Body_External.SetX(InParams.InitialTM.GetLocation(), /*bInvalidate=*/false);	//do not generate wake event since this is part of initialization
	Body_External.SetR(InParams.InitialTM.GetRotation(), /*bInvalidate=*/false);
#if CHAOS_DEBUG_NAME
	Body_External.SetDebugName(MakeShareable(new FString(InParams.DebugName)));
#endif
}

void FChaosEngineInterface::ReleaseActor(FPhysicsActorHandle& Handle,FChaosScene* InScene,bool bNeverDerferRelease)
{
	if(!Handle)
	{
		UE_LOGF(LogChaos, Verbose, "Attempting to release an actor with a null handle");
		
		return;
	}

	if(InScene)
	{
		InScene->RemoveActorFromAccelerationStructure(Handle);
		RemoveActorFromSolver(Handle,InScene->GetSolver());
	}
	else
	{
		delete Handle;
	}


	Handle = nullptr;
}


FChaosScene* FChaosEngineInterface::GetCurrentScene(const FPhysicsActorHandle& InHandle)
{
	if(!InHandle)
	{
		return nullptr;
	}

	Chaos::FPBDRigidsSolver* Solver = InHandle->GetSolver<Chaos::FPBDRigidsSolver>();
	return static_cast<FChaosScene*>(Solver ? Solver->PhysSceneHack : nullptr);
}

void FChaosEngineInterface::SetGlobalPose_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FTransform& InNewPose,bool bAutoWake)
{
	Chaos::FRigidBodyHandle_External& Body_External = InActorReference->GetGameThreadAPI();
	if (!IsKinematic(InActorReference) && !IsSleeping(InActorReference) && Chaos::FVec3::IsNearlyEqual(InNewPose.GetLocation(), Body_External.X(), SMALL_NUMBER) && Chaos::FRotation3::IsNearlyEqual(InNewPose.GetRotation(), Body_External.R(), SMALL_NUMBER))
	{
		// if simulating, don't update X/R if they haven't changed. this allows scale to be set on simulating body without overriding async position/rotation.
		return;
	}

	if (IsKinematic(InActorReference))
	{
		// NOTE: SetGlobalPose is a teleport for kinematics. Use SetKinematicTarget_AssumesLocked
		// if the kinematic should calculate its velocity from the transform delta.
		Body_External.SetKinematicTarget(InNewPose);
		Body_External.SetV(FVector::Zero());
		Body_External.SetW(FVector::Zero());
	}

	Body_External.SetX(InNewPose.GetLocation());
	Body_External.SetR(InNewPose.GetRotation());

	Body_External.UpdateShapeBounds();

	if (FChaosScene* Scene = GetCurrentScene(InActorReference))
	{
		Scene->UpdateActorInAccelerationStructure(InActorReference);
	}
}

// Match the logic in places that use SyncKinematicOnGameThread (like
// FSingleParticlePhysicsProxy::PullFromPhysicsState) - to see if that will be updating the
// position. If not, then we need to do it here. 
bool ShouldSetKinematicTargetSetGameTransform(const FPhysicsActorHandle& InActorReference)
{
	Chaos::FPBDRigidParticle* Rigid = InActorReference->GetRigidParticleUnsafe();
	if (Rigid && Rigid->ObjectState() == Chaos::EObjectStateType::Kinematic)
	{
		switch (Chaos::SyncKinematicOnGameThread)
		{
		case 0:
			return true;
		case 1:
			return false;
		default:
			return !Rigid->UpdateKinematicFromSimulation();
		}
	}
	// Historically the game TM gets set through using the kinematic target even if called with a
	// non-kinematic object, so preserve that behavior.
	return true;
}

void FChaosEngineInterface::SetKinematicTarget_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FTransform& InNewTarget)
{
	// SetKinematicTarget_AssumesLocked could be called multiple times in one time step
	const Chaos::FKinematicTarget NewKinematicTarget = Chaos::FKinematicTarget::MakePositionTarget(InNewTarget);
	InActorReference->GetGameThreadAPI().SetKinematicTarget(NewKinematicTarget);

	// If enabled for this body, immediately update the body transforms to match the kinematic target.
	// @todo(chaos): Velocity is not updated here and never will be because we don't read back from the physics thread.
	// We should fix this, but it is awkward to handle multiple calls to SetKinematicTarget on the same frame if we
	// don't have a "previous transform" from which to calculate the velocity and we have overwritten X/R already.
	if (ShouldSetKinematicTargetSetGameTransform(InActorReference))
	{
		// IMPORTANT : we do not invalidate X and R as they will be properly computed using the kinematic target information 
		InActorReference->GetGameThreadAPI().SetX(InNewTarget.GetLocation(), false);
		InActorReference->GetGameThreadAPI().SetR(InNewTarget.GetRotation(), false);
		InActorReference->GetGameThreadAPI().UpdateShapeBounds();

		FChaosScene* Scene = GetCurrentScene(InActorReference);
		Scene->UpdateActorInAccelerationStructure(InActorReference);
	}
}
