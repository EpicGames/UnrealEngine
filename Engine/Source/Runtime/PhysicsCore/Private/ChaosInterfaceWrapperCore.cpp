// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosInterfaceWrapperCore.h"

#include "Chaos/Capsule.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ParticleHandle.h"
#include "PhysXPublicCore.h"

namespace ChaosInterface
{
	FORCEINLINE ECollisionShapeType ImplicitTypeToCollisionType(int32 ImplicitObjectType)
	{
		switch (ImplicitObjectType)
		{
		case Chaos::ImplicitObjectType::Sphere: return ECollisionShapeType::Sphere;
		case Chaos::ImplicitObjectType::Box: return ECollisionShapeType::Box;
		case Chaos::ImplicitObjectType::Capsule: return ECollisionShapeType::Capsule;
		case Chaos::ImplicitObjectType::Convex: return ECollisionShapeType::Convex;
		case Chaos::ImplicitObjectType::TriangleMesh: return ECollisionShapeType::Trimesh;
		case Chaos::ImplicitObjectType::HeightField: return ECollisionShapeType::Heightfield;
		default: break;
		}

		return ECollisionShapeType::None;
	}


	ECollisionShapeType GetImplicitType(const Chaos::FImplicitObject& InGeometry)
	{
		using namespace Chaos;
		int32 ResultObjectType = GetInnerType(InGeometry.GetType());
		const Chaos::FImplicitObject* CurrentGeometry = &InGeometry;

		while (CurrentGeometry && ResultObjectType == ImplicitObjectType::Transformed)
		{
			CurrentGeometry = static_cast<const TImplicitObjectTransformed<FReal, 3>*>(CurrentGeometry)->GetGeometry();
			ResultObjectType = GetInnerType(CurrentGeometry->GetType());
		}

		return ImplicitTypeToCollisionType(ResultObjectType);
	}

	Chaos::FReal GetRadius(const Chaos::FCapsule& InCapsule)
	{
		return InCapsule.GetRadiusf();
	}

	Chaos::FReal GetHalfHeight(const Chaos::FCapsule& InCapsule)
	{
		return InCapsule.GetHeightf() / 2.;
	}

	FCollisionFilterData GetQueryFilterData(const Chaos::FPerShapeData& Shape)
	{
		return Chaos::Filter::FShapeFilterBuilder::GetLegacyShapeQueryFilter(GetCombinedShapeFilterData(Shape));
	}

	FCollisionFilterData GetSimulationFilterData(const Chaos::FPerShapeData& Shape)
	{
		return Chaos::Filter::FShapeFilterBuilder::GetLegacyShapeSimFilter(GetCombinedShapeFilterData(Shape));
	}

	Chaos::Filter::FShapeFilterData GetShapeFilterData(const Chaos::FPerShapeData& Shape)
	{
		return Shape.GetShapeFilterData();
	}

	Chaos::Filter::FInstanceData GetFilterInstanceData(const Chaos::FPerShapeData& Shape)
	{
		return Shape.GetFilterInstanceData();
	}

	Chaos::Filter::FCombinedShapeFilterData GetCombinedShapeFilterData(const Chaos::FPerShapeData& Shape)
	{
		return Shape.GetCombinedShapeFilterData();
	}

}
