// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ChaosPhysicalMaterial.cpp
=============================================================================*/ 

#include "Chaos/ChaosPhysicalMaterial.h"
#include "Chaos/Defines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosPhysicalMaterial)

UChaosPhysicalMaterial::UChaosPhysicalMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Initialize values from FChaosPhysicsMaterial
	Chaos::FChaosPhysicsMaterial TmpMat;

	Friction = TmpMat.Friction;
	StaticFriction = TmpMat.StaticFriction;
	Restitution = TmpMat.Restitution;
	LinearEtherDrag = TmpMat.LinearEtherDrag;
	AngularEtherDrag = TmpMat.AngularEtherDrag;
	SleepingAngularVelocityThreshold = TmpMat.SleepingAngularThreshold;
	SleepingLinearVelocityThreshold = TmpMat.SleepingLinearThreshold;
}

void UChaosPhysicalMaterial::CopyTo(Chaos::FChaosPhysicsMaterial& Mat) const
{
	Mat.Friction = Friction;
	Mat.StaticFriction = StaticFriction;
	Mat.Restitution = Restitution;
	Mat.LinearEtherDrag = LinearEtherDrag;
	Mat.AngularEtherDrag = AngularEtherDrag;
	Mat.SleepingLinearThreshold = SleepingLinearVelocityThreshold;
	Mat.SleepingAngularThreshold = SleepingAngularVelocityThreshold;
}

