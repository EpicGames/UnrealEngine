// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosScene.h"

#include "Async/AsyncWork.h"
#include "Async/ParallelFor.h"

#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ChaosSolversModule.h"
#include "ChaosLog.h"
#include "ChaosStats.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVDRuntimeModule.h"
#endif

#include "Field/FieldSystem.h"

#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/SkeletalMeshPhysicsProxy.h"
#include "PhysicsProxy/StaticMeshPhysicsProxy.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/BoundingVolume.h"
#include "Chaos/Framework/DebugSubstep.h"
#include "Chaos/PerParticleGravity.h"
#include "PBDRigidActiveParticlesBuffer.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/Box.h"
#include "EventsData.h"
#include "EventManager.h"
#include "RewindData.h"
#include "PhysicsSettingsCore.h"
#include "Chaos/PhysicsSolverBaseImpl.h"
#include "Chaos/AsyncInitBodyHelper.h"

#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"

#include "ChaosDebugDraw/ChaosDDScene.h"
#include "ChaosDebugDraw/ChaosDDTimeline.h"

DECLARE_CYCLE_STAT(TEXT("Update Kinematics On Deferred SkelMeshes"), STAT_UpdateKinematicsOnDeferredSkelMeshesChaos, STATGROUP_Physics);
CSV_DEFINE_CATEGORY(ChaosPhysics,true);
CSV_DEFINE_CATEGORY(AABBTreeExpensiveStats, false);

// Stat Counters
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumDirtyAABBTreeElements"), STAT_ChaosCounter_NumDirtyAABBTreeElements, STATGROUP_ChaosCounters);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumDirtyGridOverflowElements"), STAT_ChaosCounter_NumDirtyGridOverflowElements, STATGROUP_ChaosCounters);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumDirtyElementsTooLargeForGrid"), STAT_ChaosCounter_NumDirtyElementsTooLargeForGrid, STATGROUP_ChaosCounters);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumDirtyNonEmptyCellsInGrid"), STAT_ChaosCounter_NumDirtyNonEmptyCellsInGrid, STATGROUP_ChaosCounters);

TAutoConsoleVariable<int32> CVar_ChaosSimulationEnable(TEXT("P.Chaos.Simulation.Enable"),1,TEXT("Enable / disable chaos simulation. If disabled, physics will not tick."));
TAutoConsoleVariable<int32> CVar_ApplyProjectSettings(TEXT("p.Chaos.Simulation.ApplySolverProjectSettings"), 1, TEXT("Whether to apply the solver project settings on spawning a solver"));

FChaosScene::FChaosScene(
	UObject* OwnerPtr
	, Chaos::FReal InAsyncDt
	, const FName& DebugName
)
	: SolverAccelerationStructure(nullptr)
	, ChaosModule(nullptr)
	, SceneSolver(nullptr)
	, Owner(OwnerPtr)
{
	LLM_SCOPE(ELLMTag::ChaosScene);

	ChaosModule = FChaosSolversModule::GetModule();
	check(ChaosModule);

	const bool bForceSingleThread = !(FApp::ShouldUseThreadingForPerformance() || FForkProcessHelper::SupportsMultithreadingPostFork());

	Chaos::EThreadingMode ThreadingMode = bForceSingleThread ? Chaos::EThreadingMode::SingleThread : Chaos::EThreadingMode::TaskGraph;

	SceneSolver = ChaosModule->CreateSolver(OwnerPtr, InAsyncDt, ThreadingMode, DebugName);
	check(SceneSolver);

#if WITH_CHAOS_VISUAL_DEBUGGER
	SceneSolver->GetChaosVDContextData().OwnerID = GetChaosVDContextData().Id;
	SceneSolver->GetChaosVDContextData().Id = FChaosVDRuntimeModule::Get().GenerateUniqueID();
	SceneSolver->GetChaosVDContextData().Type = static_cast<int32>(EChaosVDContextType::Solver);
#endif

	SceneSolver->PhysSceneHack = this;
	SimCallback = SceneSolver->CreateAndRegisterSimCallbackObject_External<FChaosSceneSimCallback>();

	// Apply project settings to the solver
	if (CVar_ApplyProjectSettings.GetValueOnAnyThread() != 0)
	{
		UPhysicsSettingsCore* Settings = UPhysicsSettingsCore::Get();
		SceneSolver->ApplyConfig(Settings->SolverOptions);
		SceneSolver->SetIsDeterministic(Settings->bEnableEnhancedDeterminism);
	}

	// Make sure we have initialized structure on game thread, evolution has already initialized structure, just need to copy.
	CopySolverAccelerationStructure();

}

FChaosScene::~FChaosScene()
{
	if(ensure(SceneSolver))
	{
		Chaos::FEventManager* EventManager = SceneSolver->GetEventManager();
		EventManager->UnregisterHandler(Chaos::EEventType::Collision,this);
		SceneSolver->UnregisterAndFreeSimCallbackObject_External(SimCallback);
	}

	if(ensure(ChaosModule))
	{
		// Destroy our solver
		ChaosModule->DestroySolver(GetSolver());
	}

	SimCallback = nullptr;
	ChaosModule = nullptr;
	SceneSolver = nullptr;
}

#if WITH_ENGINE
void FChaosScene::AddReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITOR
	for(auto& Obj : PieModifiedObjects)
	{
		Collector.AddReferencedObject(Obj);
	}
#endif
}
#endif

#if WITH_EDITOR
void FChaosScene::AddPieModifiedObject(UObject* InObj)
{
	if(GIsPlayInEditorWorld)
	{
		PieModifiedObjects.AddUnique(ObjectPtrWrap(InObj));
	}
}
#endif


const Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* FChaosScene::GetSpacialAcceleration() const
{
	return SolverAccelerationStructure;
}

Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* FChaosScene::GetSpacialAcceleration()
{
	return SolverAccelerationStructure;
}

void FChaosScene::CopySolverAccelerationStructure()
{
	using namespace Chaos;
	if(SceneSolver)
	{
		FPhysicsSceneGuardScopedWrite ScopedWrite(SceneSolver->GetExternalDataLock_External());
		SceneSolver->UpdateExternalAccelerationStructure_External(SolverAccelerationStructure);
	}
}

void FChaosScene::Flush()
{
	check(IsInGameThread());

	Chaos::FPBDRigidsSolver* Solver = GetSolver();

	if(Solver)
	{
		//Make sure any dirty proxy data is pushed
		Solver->AdvanceAndDispatch_External(0);	//force commands through
		Solver->WaitOnPendingTasks_External();

		// Populate the spacial acceleration
		Chaos::FPBDRigidsSolver::FPBDRigidsEvolution* Evolution = Solver->GetEvolution();

		if(Evolution)
		{
			Evolution->FlushSpatialAcceleration();
		}
	}

	CopySolverAccelerationStructure();
}

void FChaosScene::RemoveActorFromAccelerationStructure(FPhysicsActorHandle Actor)
{
	using namespace Chaos;
	RemoveActorFromAccelerationStructureImp(Actor->GetParticle_LowLevel());
}

void FChaosScene::RemoveActorFromAccelerationStructureImp(Chaos::FGeometryParticle* Particle)
{
	using namespace Chaos;
	if (GetSpacialAcceleration() && Particle->UniqueIdx().IsValid())
	{
		FPhysicsSceneGuardScopedWrite ScopedWrite(SceneSolver->GetExternalDataLock_External());
		Chaos::FAccelerationStructureHandle AccelerationHandle(Particle);
		GetSpacialAcceleration()->RemoveElementFrom(AccelerationHandle, Particle->SpatialIdx());
	}
}

void FChaosScene::UpdateActorInAccelerationStructure(const FPhysicsActorHandle& Actor)
{
	using namespace Chaos;

	if(GetSpacialAcceleration())
	{
		FPhysicsSceneGuardScopedWrite ScopedWrite(SceneSolver->GetExternalDataLock_External());

		auto SpatialAcceleration = GetSpacialAcceleration();
		const Chaos::FRigidBodyHandle_External& Body_External = Actor->GetGameThreadAPI();

		if(SpatialAcceleration)
		{

			FAABB3 WorldBounds;
			const bool bHasBounds = Body_External.GetGeometry()->HasBoundingBox();
			if(bHasBounds)
			{
				WorldBounds = Body_External.GetGeometry()->BoundingBox().TransformedAABB(FRigidTransform3(Body_External.X(), Body_External.R()));
			}


			Chaos::FAccelerationStructureHandle AccelerationHandle(Actor->GetParticle_LowLevel());
			SpatialAcceleration->UpdateElementIn(AccelerationHandle,WorldBounds,bHasBounds, Body_External.SpatialIdx());
		}

		GetSolver()->UpdateParticleInAccelerationStructure_External(Actor->GetParticle_LowLevel(), EPendingSpatialDataOperation::Update);
	}
}

void FChaosScene::UpdateActorsInAccelerationStructure(const TArrayView<FPhysicsActorHandle>& Actors)
{
	using namespace Chaos;

	if(GetSpacialAcceleration())
	{
		FPhysicsSceneGuardScopedWrite ScopedWrite(SceneSolver->GetExternalDataLock_External());

		auto SpatialAcceleration = GetSpacialAcceleration();

		if(SpatialAcceleration)
		{
			int32 NumActors = Actors.Num();
			for(int32 ActorIndex = 0; ActorIndex < NumActors; ++ActorIndex)
			{
				const FPhysicsActorHandle& Actor = Actors[ActorIndex];
				if(Actor)
				{
					const Chaos::FRigidBodyHandle_External& Body_External = Actor->GetGameThreadAPI();
					// @todo(chaos): dedupe code in UpdateActorInAccelerationStructure
					FAABB3 WorldBounds;
					const bool bHasBounds = Body_External.GetGeometry()->HasBoundingBox();
					if(bHasBounds)
					{
						WorldBounds = Body_External.GetGeometry()->BoundingBox().TransformedAABB(FRigidTransform3(Body_External.X(), Body_External.R()));
					}

					Chaos::FAccelerationStructureHandle AccelerationHandle(Actor->GetParticle_LowLevel());
					SpatialAcceleration->UpdateElementIn(AccelerationHandle,WorldBounds,bHasBounds, Body_External.SpatialIdx());
				}
			}
		}

		for(int32 ActorIndex = 0; ActorIndex < Actors.Num(); ++ActorIndex)
		{
			const FPhysicsActorHandle& Actor = Actors[ActorIndex];
			if(Actor)
			{
				GetSolver()->UpdateParticleInAccelerationStructure_External(Actor->GetParticle_LowLevel(), EPendingSpatialDataOperation::Update);
			}
		}
	}
}

void FChaosScene::AddActorsToScene_AssumesLocked(TArray<FPhysicsActorHandle>& InHandles,const bool bImmediate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChaosScene::AddActorsToScene_AssumesLocked)

	Chaos::FPhysicsSolver* Solver = GetSolver();
	UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(Solver->GetExternalDataLock_External());
	Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle,Chaos::FReal,3>* SpatialAcceleration = GetSpacialAcceleration();
	for(FPhysicsActorHandle& Handle : InHandles)
	{
		FChaosEngineInterface::AddActorToSolver(Handle,Solver);

		// Optionally add this to the game-thread acceleration structure immediately
		if(bImmediate && SpatialAcceleration)
		{
			const Chaos::FRigidBodyHandle_External& Body_External = Handle->GetGameThreadAPI();
			// Get the bounding box for the particle if it has one
			bool bHasBounds = Body_External.GetGeometry()->HasBoundingBox();
			Chaos::FAABB3 WorldBounds;
			if(bHasBounds)
			{
				const Chaos::FAABB3 LocalBounds = Body_External.GetGeometry()->BoundingBox();
				WorldBounds = LocalBounds.TransformedAABB(Chaos::FRigidTransform3(Body_External.X(), Body_External.R()));
			}

			// Insert the particle
			Chaos::FAccelerationStructureHandle AccelerationHandle(Handle->GetParticle_LowLevel());
			SpatialAcceleration->UpdateElementIn(AccelerationHandle,WorldBounds,bHasBounds, Body_External.SpatialIdx());
		}
	}
}

void FChaosSceneSimCallback::OnPreSimulate_Internal()
{
	if(const FChaosSceneCallbackInput* Input = GetConsumerInput_Internal())
	{
		// the "main" gravity is index 0
		static_cast<Chaos::FPBDRigidsSolver*>(GetSolver())->GetEvolution()->GetGravityForces().SetAcceleration(Input->Gravity, 0);
	}
}

FName FChaosSceneSimCallback::GetFNameForStatId() const
{
	const static FLazyName StaticName("FChaosSceneSimCallback");
	return StaticName;
}

void FChaosScene::SetGravity(const Chaos::FVec3& Acceleration)
{
	SimCallback->GetProducerInputData_External()->Gravity = Acceleration;
}

void FChaosScene::SetUpForFrame(const FVector* NewGrav,float InDeltaSeconds /*= 0.0f*/,float InMinPhysicsDeltaTime /*= 0.0f*/,float InMaxPhysicsDeltaTime /*= 0.0f*/,float InMaxSubstepDeltaTime /*= 0.0f*/,int32 InMaxSubsteps,bool bSubstepping)
{
	using namespace Chaos;
	SetGravity(*NewGrav);

	InDeltaSeconds *= MNetworkDeltaTimeScale;

	if(bSubstepping)
	{
		MDeltaTime = InMaxSubstepDeltaTime > 0.f ? FMath::Min(InDeltaSeconds, InMaxSubsteps * InMaxSubstepDeltaTime) : InDeltaSeconds;
	}
	else
	{
		MDeltaTime = InMaxPhysicsDeltaTime > 0.f ? FMath::Min(InDeltaSeconds, InMaxPhysicsDeltaTime) : InDeltaSeconds;
	}

	if(FPhysicsSolver* Solver = GetSolver())
	{
		if(bSubstepping)
		{
			Solver->SetMaxDeltaTime_External(InMaxSubstepDeltaTime);
			Solver->SetMaxSubSteps_External(InMaxSubsteps);
		} 
		else
		{
			Solver->SetMaxDeltaTime_External(InMaxPhysicsDeltaTime);
			Solver->SetMaxSubSteps_External(1);
		}
		Solver->SetMinDeltaTime_External(InMinPhysicsDeltaTime);
	}
}

void FChaosScene::StartFrame()
{
	using namespace Chaos;

	SCOPE_CYCLE_COUNTER(STAT_Scene_StartFrame);

	if(CVar_ChaosSimulationEnable.GetValueOnGameThread() == 0)
	{
		return;
	}

	const float UseDeltaTime = OnStartFrame(MDeltaTime);

	TArray<FPhysicsSolverBase*> SolverList = GetPhysicsSolvers();
	for(FPhysicsSolverBase* Solver : SolverList)
	{
		if(FGraphEventRef SolverEvent = Solver->AdvanceAndDispatch_External(UseDeltaTime))
		{
			if(SolverEvent.IsValid())
			{
				CompletionEvents.Add(SolverEvent);
			}
		}
	}
}

void FChaosScene::OnSyncBodies(Chaos::FPhysicsSolverBase* Solver)
{
	struct FDispatcher {} Dispatcher;
	Solver->PullPhysicsStateForEachDirtyProxy_External(Dispatcher);
}

void FChaosScene::KillSafeAsyncTasks()
{
	Chaos::FPBDRigidsSolver* Solver = GetSolver();
	if (Solver )
	{
		Solver->KillSafeAsyncTasks();
	}
}

void FChaosScene::WaitSolverTasks()
{
	Chaos::FPBDRigidsSolver* Solver = GetSolver();
	if(Solver)
	{
		Solver->WaitOnPendingTasks_External();
	}
}

bool FChaosScene::AreAnyTasksPending() const
{
	const Chaos::FPBDRigidsSolver* Solver = GetSolver();
	if (Solver && Solver->AreAnyTasksPending())
	{
		return true;
	}
	
	return false;
}

void FChaosScene::BeginDestroy()
{
	Chaos::FPBDRigidsSolver* Solver = GetSolver();
	if (Solver)
	{
		Solver->BeginDestroy();
	}
}

bool FChaosScene::IsCompletionEventComplete() const
{
	for (FGraphEventRef Event : CompletionEvents)
	{
		if (Event && !Event->IsComplete())
		{
			return false;
		}
	}

	return true;
}

template <typename TSolver>
void FChaosScene::SyncBodies(TSolver* Solver)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("SyncBodies"),STAT_SyncBodies,STATGROUP_Physics);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(SyncBodies);
	OnSyncBodies(Solver);
}

// Accumulate all the AABBTree stats
void GetAABBTreeStats(Chaos::ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>& Collection, Chaos::AABBTreeStatistics& OutAABBTreeStatistics, Chaos::AABBTreeExpensiveStatistics& OutAABBTreeExpensiveStatistics)
{
	CSV_SCOPED_TIMING_STAT(AABBTreeExpensiveStats, GetAABBTreeStats);
	using namespace Chaos;
	OutAABBTreeStatistics.Reset();
	TArray<FSpatialAccelerationIdx> SpatialIndices = Collection.GetAllSpatialIndices();
	for (const FSpatialAccelerationIdx SpatialIndex : SpatialIndices)
	{
		auto SubStructure = Collection.GetSubstructure(SpatialIndex);
		if (const auto AABBTree = SubStructure->template As<TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>>())
		{
			OutAABBTreeStatistics.MergeStatistics(AABBTree->GetAABBTreeStatistics());
#if CSV_PROFILER_STATS
			if (FCsvProfiler::Get()->IsCapturing() && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(AABBTreeExpensiveStats)))
			{
				OutAABBTreeExpensiveStatistics.MergeStatistics(AABBTree->GetAABBTreeExpensiveStatistics());
			}
#endif
		}
		else if (const auto AABBTreeBV = SubStructure->template As<TAABBTree<FAccelerationStructureHandle, TBoundingVolume<FAccelerationStructureHandle>>>())
		{
			OutAABBTreeStatistics.MergeStatistics(AABBTreeBV->GetAABBTreeStatistics());
		}
	}
}

TArray<Chaos::FPhysicsSolverBase*> FChaosScene::GetPhysicsSolvers() const
{
	// Make a list of solvers to process. This is a list of all solvers registered to our world
	// And our internal base scene solver.
	TArray<Chaos::FPhysicsSolverBase*> SolverList;
	if(const Chaos::FPhysicsSolver* Solver = GetSolver())
	{
		if(!Solver->IsStandaloneSolver())
		{
			// Get all the solvers with the same owner
			ChaosModule->GetSolversMutable(Owner,SolverList);
		}

		// Make sure our solver is in the list
		SolverList.AddUnique(GetSolver());
	}
	return SolverList;
}

void FChaosScene::EndFrame()
{
	using namespace Chaos;
	using SpatialAccelerationCollection = ISpatialAccelerationCollection<FAccelerationStructureHandle,FReal,3>;

	SCOPE_CYCLE_COUNTER(STAT_Scene_EndFrame);

	if(CVar_ChaosSimulationEnable.GetValueOnGameThread() == 0 || GetSolver() == nullptr)
	{
		return;
	}

	CVD_TRACE_ACCELERATION_STRUCTURES(SolverAccelerationStructure, Chaos::FPhysicsSolver, *SceneSolver, CVDDC_AccelerationStructures);

#if !UE_BUILD_SHIPPING
	{
		Chaos::AABBTreeStatistics TreeStats;
		Chaos::AABBTreeExpensiveStatistics TreeExpensiveStats;
		GetAABBTreeStats(GetSpacialAcceleration()->AsChecked<SpatialAccelerationCollection>(), TreeStats, TreeExpensiveStats);

		CSV_CUSTOM_STAT(ChaosPhysics, AABBTreeDirtyElementCount, TreeStats.StatNumDirtyElements, ECsvCustomStatOp::Set);
		SET_DWORD_STAT(STAT_ChaosCounter_NumDirtyAABBTreeElements, TreeStats.StatNumDirtyElements);

		CSV_CUSTOM_STAT(ChaosPhysics, AABBTreeDirtyGridOverflowCount, TreeStats.StatNumGridOverflowElements, ECsvCustomStatOp::Set);
		SET_DWORD_STAT(STAT_ChaosCounter_NumDirtyGridOverflowElements, TreeStats.StatNumGridOverflowElements);

		CSV_CUSTOM_STAT(ChaosPhysics, AABBTreeDirtyElementTooLargeCount, TreeStats.StatNumElementsTooLargeForGrid, ECsvCustomStatOp::Set);
		SET_DWORD_STAT(STAT_ChaosCounter_NumDirtyElementsTooLargeForGrid, TreeStats.StatNumElementsTooLargeForGrid);

		CSV_CUSTOM_STAT(ChaosPhysics, AABBTreeDirtyElementNonEmptyCellCount, TreeStats.StatNumNonEmptyCellsInGrid, ECsvCustomStatOp::Set);
		SET_DWORD_STAT(STAT_ChaosCounter_NumDirtyNonEmptyCellsInGrid, TreeStats.StatNumNonEmptyCellsInGrid);

#if CSV_PROFILER_STATS
		if (FCsvProfiler::Get()->IsCapturing() && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(AABBTreeExpensiveStats)))
		{
			CSV_CUSTOM_STAT(AABBTreeExpensiveStats, AABBTreeMaxNumLeaves, TreeExpensiveStats.StatMaxNumLeaves, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(AABBTreeExpensiveStats, AABBTreeMaxDirtyElements, TreeExpensiveStats.StatMaxDirtyElements, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(AABBTreeExpensiveStats, AABBTreeMaxTreeDepth, TreeExpensiveStats.StatMaxTreeDepth, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(AABBTreeExpensiveStats, AABBTreeMaxLeafSize, TreeExpensiveStats.StatMaxLeafSize, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(AABBTreeExpensiveStats, AABBTreeGlobalPayloadsSize, TreeExpensiveStats.StatGlobalPayloadsSize, ECsvCustomStatOp::Set);
		}
#endif // CSV_PROFILER_STATS
	}
#endif // UE_BUILD_SHIPPING

	check(IsCompletionEventComplete())
	//check(PhysicsTickTask->IsComplete());
	CompletionEvents.Reset();

	// Make a list of solvers to process. This is a list of all solvers registered to our world
	// And our internal base scene solver.
	TArray<FPhysicsSolverBase*> SolverList = GetPhysicsSolvers();

	// Flip the buffers over to the game thread and sync
	{
		SCOPE_CYCLE_COUNTER(STAT_FlipResults);

		//update external SQ structure
		//for now just copy the whole thing, stomping any changes that came from GT
		CopySolverAccelerationStructure();

		for(FPhysicsSolverBase* Solver : SolverList)
		{
			Solver->CastHelper([&SolverList, Solver, this](auto& Concrete)
			{
				SyncBodies(&Concrete);
				Solver->FlipEventManagerBuffer();
				Concrete.SyncEvents_GameThread();
				{
					SCOPE_CYCLE_COUNTER(STAT_SqUpdateMaterials);
					Concrete.SyncQueryMaterials_External();
				}
			});
		}
	}

	OnPhysScenePostTick.Broadcast(this);
}

void FChaosScene::WaitPhysScenes()
{
	if(!IsCompletionEventComplete())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FPhysScene_WaitPhysScenes);
		FTaskGraphInterface::Get().WaitUntilTasksComplete(CompletionEvents,ENamedThreads::GameThread);
	}

	// CompletionEvents only captures the return value of AdvanceAndDispatch_External, which in
	// async physics mode (IsUsingAsyncResults()==true) intentionally returns the event from the
	// previous frame rather than the newly-dispatched tasks. Those tasks run independently and
	// are not tracked in CompletionEvents. WaitOnPendingTasks_External drains them directly.
	for (Chaos::FPhysicsSolverBase* Solver : GetPhysicsSolvers())
	{
		if (Solver && Solver->IsUsingAsyncResults())
		{
			Solver->WaitOnPendingTasks_External();
		}
	}
}

FGraphEventArray FChaosScene::GetCompletionEvents()
{
	return CompletionEvents;
}

#if CHAOS_DEBUG_DRAW
void FChaosScene::SetDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& InCDDScene)
{
	CDDScene = InCDDScene;

	SceneSolver->EnqueueCommandImmediate(
		[this]()
		{
			SceneSolver->SetDebugDrawScene(CDDScene);
		});
}
#endif
