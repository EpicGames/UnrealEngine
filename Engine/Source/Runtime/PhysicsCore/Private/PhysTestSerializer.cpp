// Copyright Epic Games, Inc. All Rights Reserved.

// Physics engine integration utilities

#include "PhysTestSerializer.h"

#include "PhysicsCore.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Capsule.h"

#include "PhysicsPublicCore.h"
#include "PhysicsCore.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"

FPhysTestSerializer::FPhysTestSerializer()
	: bDiskDataIsChaos(false)
	, bChaosDataReady(false)
	, Particles(UniqueIndices),
	ChaosContext(MakeShared<Chaos::FChaosArchiveContext>())
{
}

void FPhysTestSerializer::Serialize(const TCHAR* FilePrefix)
{
	check(IsInGameThread());
	int32 Tries = 0;
	FString UseFileName;
	const FString FullPathPrefix = FPaths::ProfilingDir() / FilePrefix;
	do
	{
		UseFileName = FString::Printf(TEXT("%s_%d.bin"), *FullPathPrefix, Tries++);
	} while (IFileManager::Get().FileExists(*UseFileName));

	//this is not actually file safe but oh well, very unlikely someone else is trying to create this file at the same time
	TUniquePtr<FArchive> File(IFileManager::Get().CreateFileWriter(*UseFileName));
	if (File)
	{
		UE_LOGF(LogPhysicsCore, Log, "PhysTestSerialize File: %ls", *UseFileName);
		Serialize(*File);
	}
	else
	{
		UE_LOGF(LogPhysicsCore, Warning, "Could not create PhysTestSerialize file(%ls)", *UseFileName);
	}
}

void FPhysTestSerializer::Serialize(FArchive& Ar)
{
	TUniquePtr<Chaos::FChaosArchive> ChaosAr = MakeUnique<Chaos::FChaosArchive>(Ar);
	
	if (!Ar.IsLoading())
	{
		//make sure any context we had set is restored before writing out sqcapture
		ChaosAr = MakeUnique<FArchiveWithExternalContext>(Ar, ChaosContext);
	}

	static const FName TestSerializerName = "PhysTestSerializer";

	{
		Chaos::FChaosArchiveScopedMemory ScopedMemory(*ChaosAr, TestSerializerName, false);
		int Version = 1;
		Ar << Version;
		Ar << bDiskDataIsChaos;

		if (Version >= 1)
		{
			//use version recorded
			ArchiveVersion.Serialize(Ar);
		}
		else
		{
			//no version recorded so use the latest versions in GUIDs we rely on before making serialization version change
			ArchiveVersion.SetVersion(FPhysicsObjectVersion::GUID, FPhysicsObjectVersion::SerializeGTGeometryParticles, TEXT("SerializeGTGeometryParticles"));
			ArchiveVersion.SetVersion(FDestructionObjectVersion::GUID, FDestructionObjectVersion::GroupAndAttributeNameRemapping, TEXT("GroupAndAttributeNameRemapping"));
			ArchiveVersion.SetVersion(FExternalPhysicsCustomObjectVersion::GUID, FExternalPhysicsCustomObjectVersion::BeforeCustomVersionWasAdded, TEXT("BeforeCustomVersionWasAdded"));
		}
		
		Ar.SetCustomVersions(ArchiveVersion);
		Ar << Data;
	}

	if (Ar.IsLoading())
	{
#if 0
		CreateChaosData();
#endif

		ChaosAr = MakeUnique<FArchiveWithExternalContext>(Ar, ChaosContext);	//make sure any context we created during load is used for sqcapture
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bHasSQCapture = !!SQCapture;
	{
		Chaos::FChaosArchiveScopedMemory ScopedMemory(*ChaosAr, TestSerializerName, false);
		Ar << bHasSQCapture;
	}
	if(bHasSQCapture)
	{
		if (Ar.IsLoading())
		{
			SQCapture = TUniquePtr<FSQCapture>(new FSQCapture(*this));
		}
		SQCapture->Serialize(*ChaosAr);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FPhysTestSerializer::Serialize(Chaos::FChaosArchive& Ar)
{
	// No longer supported
	ensure(false);
}

void FPhysTestSerializer::SetPhysicsData(Chaos::FPBDRigidsEvolution& Evolution)
{
	bDiskDataIsChaos = true;
	Data.Empty();
	FMemoryWriter Ar(Data);

	TSharedRef<Chaos::FChaosArchiveContext> NewExternalContext = MakeShared<Chaos::FChaosArchiveContext>();
	FArchiveWithExternalContext ChaosAr(Ar, NewExternalContext);
	Evolution.Serialize(ChaosAr);
	ChaosContext = NewExternalContext;
	ArchiveVersion = Ar.GetCustomVersions();
}

#if 0

void FPhysTestSerializer::CreateChaosData()
{
	if (bDiskDataIsChaos == false)
	{
		if (bChaosDataReady)
		{
			return;
		}

		PxScene* Scene = GetPhysXData();
		check(Scene);

		const uint32 NumStatic = Scene->getNbActors(PxActorTypeFlag::eRIGID_STATIC);
		const uint32 NumDynamic = Scene->getNbActors(PxActorTypeFlag::eRIGID_DYNAMIC);
		const uint32 NumActors = NumStatic + NumDynamic;

		TArray<PxActor*> Actors;
		Actors.AddUninitialized(NumActors);
		if (NumStatic)
		{
			Scene->getActors(PxActorTypeFlag::eRIGID_STATIC, Actors.GetData(), NumStatic);
			auto NewParticles = Particles.CreateStaticParticles(NumStatic);	//question: do we want to distinguish query only and sim only actors?
			for (uint32 Idx = 0; Idx < NumStatic; ++Idx)
			{
				GTParticles.Emplace(FGeometryParticle::CreateParticle());
				NewParticles[Idx]->GTGeometryParticle() = GTParticles.Last().Get();
			}
		}

		if (NumDynamic)
		{
			Scene->getActors(PxActorTypeFlag::eRIGID_DYNAMIC, &Actors[NumStatic], NumDynamic);
			auto NewParticles = Particles.CreateDynamicParticles(NumDynamic);	//question: do we want to distinguish query only and sim only actors?

			for (uint32 Idx = 0; Idx < NumDynamic; ++Idx)
			{
				GTParticles.Emplace(FPBDRigidParticle::CreateParticle());
				NewParticles[Idx]->GTGeometryParticle() = GTParticles.Last().Get();
			}
		}

		auto& Handles = Particles.GetParticleHandles();
		int32 Idx = 0;
		for (PxActor* Act : Actors)
		{
			//transform
			PxRigidActor* Actor = static_cast<PxRigidActor*>(Act);
			auto& Particle = Handles.Handle(Idx);
			auto& GTParticle = Particle->GTGeometryParticle();
			Particle->X() = P2UVector(Actor->getGlobalPose().p);
			Particle->R() = P2UQuat(Actor->getGlobalPose().q);
			Particle->GTGeometryParticle()->SetX(Particle->X());
			Particle->GTGeometryParticle()->SetR(Particle->R());

			auto PBDRigid = Particle->CastToRigidParticle();
			if(PBDRigid && PBDRigid->ObjectState() == EObjectStateType::Dynamic)
			{
				PBDRigid->P() = Particle->X();
				PBDRigid->Q() = Particle->R();
			}

			PxActorToChaosHandle.Add(Act, Particle.Get());

			//geometry
			TArray<Chaos::FImplicitObjectPtr> Geoms;
			const int32 NumShapes = Actor->getNbShapes();
			TArray<PxShape*> Shapes;
			Shapes.AddUninitialized(NumShapes);
			Actor->getShapes(Shapes.GetData(), NumShapes);
			for (PxShape* Shape : Shapes)
			{
				if (TUniquePtr<TImplicitObjectTransformed<float, 3>> Geom = PxShapeToChaosGeom(Shape))
				{
					Geoms.Add(MoveTemp(Geom));
				}
			}

			if (Geoms.Num())
			{
				if (Geoms.Num() == 1)
				{
					auto SharedGeom = FImplicitObjectPtr(Geoms[0]);
					GTParticle->SetGeometry(SharedGeom);
					Particle->SetSharedGeometry(SharedGeom);
				}
				else
				{
					GTParticle->SetGeometry(MakeUnique<FImplicitObjectUnion>(MoveTemp(Geoms)));
					Particle->SetGeometry(GTParticle->Geometry());
				}

				// Fixup bounds
				auto Geom = GTParticle->Geometry();
				if (Geom->HasBoundingBox())
				{
					auto& ShapeArray = GTParticle->ShapesArray();
					for (auto& Shape : ShapeArray)
					{
						Shape->SetWorldSpaceInflatedShapeBounds(Geom->BoundingBox().TransformedAABB(TRigidTransform<FReal, 3>(Particle->X(), Particle->R())));
					}
				}
			}

			int32 ShapeIdx = 0;
			for (PxShape* Shape : Shapes)
			{
				PxShapeToChaosShapes.Add(Shape, GTParticle->ShapesArray()[ShapeIdx++].Get());
			}

			++Idx;
		}

		ChaosEvolution = MakeUnique<Chaos::FPBDRigidsEvolution>(Particles, PhysicalMaterials);
	}
	else
	{
		ChaosEvolution = MakeUnique<Chaos::FPBDRigidsEvolution>(Particles, PhysicalMaterials);

		FMemoryReader Ar(Data);
		TSharedRef<Chaos::FChaosArchiveContext> NewExternalContext = MakeShared<Chaos::FChaosArchiveContext>();
		FArchiveWithExternalContext ChaosAr(Ar, NewExternalContext);

		Ar.SetCustomVersions(ArchiveVersion);

		ChaosEvolution->Serialize(ChaosAr);
		ChaosContext = NewExternalContext;
	}
	bChaosDataReady = true;
}
#endif
