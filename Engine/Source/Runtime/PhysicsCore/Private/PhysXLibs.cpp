// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysXLibs.cpp: PhysX library imports
=============================================================================*/
#include "PhysicsCore.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

// PhysX library imports
namespace PhysDLLHelper
{
	const static int32 NumModuleLoadRetries = 5;
	const static float ModuleReloadDelay = 0.5f;

#if PLATFORM_WINDOWS || PLATFORM_MAC
	void* PxFoundationHandle = nullptr;
	void* PhysX3CommonHandle = nullptr;
	void* PhysX3Handle = nullptr;
	void* PxPvdSDKHandle = nullptr;
	void* PhysX3CookingHandle = nullptr;
	void* nvToolsExtHandle = nullptr;
#endif

#if PLATFORM_WINDOWS

	FString PhysXBinariesRoot = "Binaries/ThirdParty/PhysX3/";
	FString APEXBinariesRoot = "Binaries/ThirdParty/PhysX3/";
	FString SharedBinariesRoot = "Binaries/ThirdParty/PhysX3/";

#if _MSC_VER >= 1900
	FString VSDirectory("VS2015/");
#else
#error "Unrecognized Visual Studio version."
#endif

	FString RootPhysXPath(PhysXBinariesRoot + TEXT("Win64/") + VSDirectory);
	FString RootAPEXPath(APEXBinariesRoot + TEXT("Win64/") + VSDirectory);
	FString RootSharedPath(SharedBinariesRoot + TEXT("Win64/") + VSDirectory);

#if PLATFORM_CPU_X86_FAMILY
	FString ArchName("_x64");
	FString ArchBits("64");
#elif PLATFORM_CPU_ARM_FAMILY
	FString ArchName("_arm64");
	FString ArchBits("arm64");
#endif

#ifdef UE_PHYSX_SUFFIX
	FString PhysXSuffix(TEXT(UE_STRINGIZE(UE_PHYSX_SUFFIX)) + ArchName + TEXT(".dll"));
#else
	FString PhysXSuffix(ArchName + TEXT(".dll"));
#endif

#ifdef UE_APEX_SUFFIX
	FString APEXSuffix(TEXT(UE_STRINGIZE(UE_APEX_SUFFIX)) + ArchName + TEXT(".dll"));
#else
	FString APEXSuffix(ArchName + TEXT(".dll"));
#endif
#elif PLATFORM_MAC
	FString PhysXBinariesRoot = "Binaries/ThirdParty/PhysX3/Mac/";
#ifdef UE_PHYSX_SUFFIX
	FString PhysXSuffix = FString(TEXT(UE_STRINGIZE(UE_PHYSX_SUFFIX))) + TEXT(".dylib");
#else
	FString PhysXSuffix(".dylib");
#endif

#ifdef UE_APEX_SUFFIX
	FString APEXSuffix = FString(TEXT(UE_STRINGIZE(UE_APEX_SUFFIX))) + TEXT(".dylib");
#else
	FString APEXSuffix(".dylib");
#endif
#endif

void* LoadPhysicsLibrary(const FString& PathEnd)
{
	const FString Path = FPaths::EngineDir() / PathEnd;
	void* Handle = FPlatformProcess::GetDllHandle(*Path);
	if(!Handle)
	{
		// Spin a few times and reattempt the load in-case the file is temporarily locked
		for(int32 RetryCount = 0; RetryCount < NumModuleLoadRetries; ++RetryCount)
		{
			FPlatformProcess::Sleep(ModuleReloadDelay);

			Handle = FPlatformProcess::GetDllHandle(*Path);

			if(Handle)
			{
				break;
			}
		}

		if(!Handle)
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

			bool bExists = PlatformFile.FileExists(*Path);
			int64 ModuleFileSize = PlatformFile.FileSize(*Path);

			bool bCouldRead = false;

			TUniquePtr<IFileHandle> ModuleFileHandle(PlatformFile.OpenRead(*Path));
			if(ModuleFileHandle.IsValid())
			{
				bCouldRead = true;
			}

			UE_LOGF(LogPhysicsCore, Warning, "Failed to load module '%ls'", *Path);
			UE_LOGF(LogPhysicsCore, Warning, "\tExists: %ls", bExists ? TEXT("true") : TEXT("false"));
			UE_LOGF(LogPhysicsCore, Warning, "\tFileSize: %lld", ModuleFileSize);
			UE_LOGF(LogPhysicsCore, Warning, "\tAble to read: %ls", bCouldRead ? TEXT("true") : TEXT("false"));

			if(!bExists)
			{
				// No library
				UE_LOGF(LogPhysicsCore, Warning, "\tLibrary does not exist.");
			}
			else if(!bCouldRead)
			{
				// No read access to library
				UE_LOGF(LogPhysicsCore, Warning, "\tLibrary exists, but read access could not be gained. It is possible the user does not have read permission for this file.");
			}
		}
	}
	return Handle;
}

/**
 *	Load the required modules for PhysX
 */
PHYSICSCORE_API bool LoadPhysXModules(bool bLoadCookingModule)
{
	bool bHasToolsExtensions = false;
#if PLATFORM_WINDOWS
	PxFoundationHandle = LoadPhysicsLibrary(RootSharedPath + "PxFoundation" + PhysXSuffix);
	PhysX3CommonHandle = LoadPhysicsLibrary(RootPhysXPath + "PhysX3Common" + PhysXSuffix);
	const FString nvToolsExtPath = RootPhysXPath + "nvToolsExt" + ArchBits + "_1.dll";
	bHasToolsExtensions = IFileManager::Get().FileExists(*nvToolsExtPath);
	if (bHasToolsExtensions)
	{
		nvToolsExtHandle = LoadPhysicsLibrary(nvToolsExtPath);
	}
	PxPvdSDKHandle = LoadPhysicsLibrary(RootSharedPath + "PxPvdSDK" + PhysXSuffix);
	PhysX3Handle = LoadPhysicsLibrary(RootPhysXPath + "PhysX3" + PhysXSuffix);

	if(bLoadCookingModule)
	{
		PhysX3CookingHandle = LoadPhysicsLibrary(RootPhysXPath + "PhysX3Cooking" + PhysXSuffix);
	}

#elif PLATFORM_MAC
	const FString PxFoundationLibName = FString::Printf(TEXT("%slibPxFoundation%s"), *PhysXBinariesRoot, *PhysXSuffix);
	PxFoundationHandle = LoadPhysicsLibrary(PxFoundationLibName);

	const FString PhysX3CommonLibName = FString::Printf(TEXT("%slibPhysX3Common%s"), *PhysXBinariesRoot, *PhysXSuffix);
	PhysX3CommonHandle = LoadPhysicsLibrary(PhysX3CommonLibName);

	const FString PxPvdSDKLibName = FString::Printf(TEXT("%slibPxPvdSDK%s"), *PhysXBinariesRoot, *PhysXSuffix);
	PxPvdSDKHandle = LoadPhysicsLibrary(PxPvdSDKLibName);

	const FString PhysX3LibName = FString::Printf(TEXT("%slibPhysX3%s"), *PhysXBinariesRoot, *PhysXSuffix);
	PhysX3Handle = LoadPhysicsLibrary(PhysX3LibName);

	if(bLoadCookingModule)
	{
		const FString PhysX3CookinLibName = FString::Printf(TEXT("%slibPhysX3Cooking%s"), *PhysXBinariesRoot, *PhysXSuffix);
		PhysX3CookingHandle = LoadPhysicsLibrary(PhysX3CookinLibName);
	}
#endif	//PLATFORM_WINDOWS

	bool bSucceeded = true;

#if PLATFORM_WINDOWS || PLATFORM_MAC
	// Required modules (core PhysX)
	bSucceeded = bSucceeded && PxFoundationHandle;
	bSucceeded = bSucceeded && PhysX3CommonHandle;
	bSucceeded = bSucceeded && PxPvdSDKHandle;
	bSucceeded = bSucceeded && PhysX3Handle;
	// Tools extension if present
	bSucceeded = bSucceeded && (!bHasToolsExtensions || nvToolsExtHandle);
	// Cooking module if present
	bSucceeded = bSucceeded && (!bLoadCookingModule || PhysX3CookingHandle);
	// Apex if present
#endif // PLATFORM_WINDOWS || PLATFORM_MAC

	return bSucceeded;
}

/** 
 *	Unload the required modules for PhysX
 */
PHYSICSCORE_API void UnloadPhysXModules()
{
#if PLATFORM_WINDOWS || PLATFORM_MAC
	FPlatformProcess::FreeDllHandle(PxPvdSDKHandle);
	FPlatformProcess::FreeDllHandle(PhysX3Handle);
	if(PhysX3CookingHandle)
	{
		FPlatformProcess::FreeDllHandle(PhysX3CookingHandle);
	}
	FPlatformProcess::FreeDllHandle(PhysX3CommonHandle);
	FPlatformProcess::FreeDllHandle(PxFoundationHandle);
#endif
}
}
