// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicalMaterial.cpp
=============================================================================*/ 

#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicalMaterials/PhysicalMaterialPropertyBase.h"
#include "UObject/UObjectIterator.h"
#include "Chaos/AsyncInitBodyHelper.h"
#include "Chaos/PhysicalMaterials.h"
#include "HAL/IConsoleManager.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicalMaterial)

namespace PhysicalMaterialCVars
{
	bool bShowExperimentalProperties = false;

	FAutoConsoleVariableRef CVarShowExperimentalProperties(TEXT("p.PhysicalMaterial.ShowExperimentalProperties"), bShowExperimentalProperties, TEXT(""));
}

UDEPRECATED_PhysicalMaterialPropertyBase::UDEPRECATED_PhysicalMaterialPropertyBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FPhysicalMaterialStrength::FPhysicalMaterialStrength()
{
	// using concrete as default ( lowest values of it )
	TensileStrength = 2;
	CompressionStrength = 20;
	ShearStrength = 6;
}

FPhysicalMaterialDamageModifier::FPhysicalMaterialDamageModifier()
{
	DamageThresholdMultiplier = 1.0;
}

UPhysicalMaterial::UPhysicalMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Friction = 0.7f;
	StaticFriction = 0.f;
	Restitution = 0.3f;
	RaiseMassToPower = 0.75f;
	Density = 1.0f;
	SleepLinearVelocityThreshold = 1.f;
	SleepAngularVelocityThreshold = 0.05f;
	SleepCounterThreshold = 4;
	bOverrideFrictionCombineMode = false;
	UserData = FChaosUserData(this);

	SoftCollisionMode = EPhysicalMaterialSoftCollisionMode::None;
	SoftCollisionThickness = 0;

	BaseFrictionImpulse = 0;
}

UPhysicalMaterial::UPhysicalMaterial(FVTableHelper& Helper)
	: Super(Helper)
{
}

UPhysicalMaterial::~UPhysicalMaterial() = default;

#if WITH_EDITOR
void UPhysicalMaterial::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bSkipUpdate = false;
	if(!MaterialHandle)
	{
		// If we don't currently have a material calling GetPhysicsMaterial will already call update as a side effect
		// to set the initial state - so we can skip it in that case.
		bSkipUpdate = true;
	}

	FPhysicsMaterialHandle& PhysMaterial = GetPhysicsMaterial();

	if(!bSkipUpdate)
	{
		FChaosEngineInterface::UpdateMaterial(*MaterialHandle, this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UPhysicalMaterial::RebuildPhysicalMaterials()
{
	for (FThreadSafeObjectIterator Iter(UPhysicalMaterial::StaticClass()); Iter; ++Iter)
	{
		if (UPhysicalMaterial * PhysicalMaterial = Cast<UPhysicalMaterial>(*Iter))
		{
			if(!PhysicalMaterial->MaterialHandle)
			{
				PhysicalMaterial->MaterialHandle = MakeUnique<FPhysicsMaterialHandle>();
			}
			FChaosEngineInterface::UpdateMaterial(*PhysicalMaterial->MaterialHandle, PhysicalMaterial);
		}
	}
}

#endif // WITH_EDITOR

void UPhysicalMaterial::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Super::Serialize(Ar);
}

void UPhysicalMaterial::PostLoad()
{
	Super::PostLoad();

	// we're removing physical material property, so convert to Material type
	if (GetLinkerUEVersion() < VER_UE4_REMOVE_PHYSICALMATERIALPROPERTY)
	{
		if (PhysicalMaterialProperty_DEPRECATED)
		{
			SurfaceType = PhysicalMaterialProperty_DEPRECATED->ConvertToSurfaceType();
		}
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::DebugColorForPhysicalMaterials)
	{
		// We don't have a color yet, generate one based on a stable hash generated by the asset's path : 
		DebugColor = FLinearColor::MakeRandomSeededColor(GetTypeHash(GetPathName()));
	}

	if (Chaos::CVars::bEnableAsyncInitBody)
	{
		// Prime material handle for thread-safe access in async physics path (i.e. when p.Chaos.EnableAsyncInitBody = true)
		EnsurePhysicsMaterialCreated();
	}
}

void UPhysicalMaterial::FinishDestroy()
{
	if(MaterialHandle)
	{
		FChaosEngineInterface::ReleaseMaterial(*MaterialHandle);
	}
	Super::FinishDestroy();
}

void UPhysicalMaterial::EnsurePhysicsMaterialCreated()
{
	if(!MaterialHandle)
	{
		MaterialHandle = MakeUnique<FPhysicsMaterialHandle>();
	}
	if(!MaterialHandle->IsValid())
	{
		*MaterialHandle = FChaosEngineInterface::CreateMaterial(this);
		check(MaterialHandle->IsValid());

		FChaosEngineInterface::SetUserData(*MaterialHandle, &UserData);
		FChaosEngineInterface::UpdateMaterial(*MaterialHandle, this);
	}
}

FPhysicsMaterialHandle& UPhysicalMaterial::GetPhysicsMaterial()
{
	EnsurePhysicsMaterialCreated();
	return *MaterialHandle;
}

//This is a bit of a hack, should probably just have a default material live in PhysicsCore instead of in Engine
static UPhysicalMaterial* GEngineDefaultPhysMaterial = nullptr;

void UPhysicalMaterial::SetEngineDefaultPhysMaterial(UPhysicalMaterial* Material)
{
	GEngineDefaultPhysMaterial = Material;
}

static UPhysicalMaterial* GEngineDefaultDestructiblePhysMaterial = nullptr;

void UPhysicalMaterial::SetEngineDefaultDestructiblePhysMaterial(UPhysicalMaterial* Material)
{
	GEngineDefaultDestructiblePhysMaterial = Material;
}

EPhysicalSurface UPhysicalMaterial::DetermineSurfaceType(UPhysicalMaterial const* PhysicalMaterial)
{
	if (PhysicalMaterial == NULL)
	{
		PhysicalMaterial = GEngineDefaultPhysMaterial;
	}
	return PhysicalMaterial->SurfaceType;
}

