// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsCore.h"
#include "Interface_CollisionDataProviderCore.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

DEFINE_LOG_CATEGORY(LogPhysicsCore);

#if WITH_EDITORONLY_DATA
void operator<<(class FArchive& Ar, FTriIndices& Indices)
{
	Ar << Indices.v0 << Indices.v1 << Indices.v2;
}

void operator<<(class FArchive& Ar, FTriMeshCollisionData& TriMeshCollisionData)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Ar << TriMeshCollisionData.Vertices;
	Ar << TriMeshCollisionData.Indices;
	Ar << TriMeshCollisionData.MaterialIndices;
	Ar << TriMeshCollisionData.UVs;
	Ar << TriMeshCollisionData.Vertices;

	FArchive_Serialize_BitfieldBool(Ar, TriMeshCollisionData.bFlipNormals);
	FArchive_Serialize_BitfieldBool(Ar, TriMeshCollisionData.bDeformableMesh);
	FArchive_Serialize_BitfieldBool(Ar, TriMeshCollisionData.bFastCook);
	FArchive_Serialize_BitfieldBool(Ar, TriMeshCollisionData.bDisableActiveEdgePrecompute);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::OptionalQuantizePerFaceBoundsInChaosTriMeshBVH)
	{
		FArchive_Serialize_BitfieldBool(Ar, TriMeshCollisionData.bReducedMemoryRepresentation);
	}
}
#endif
