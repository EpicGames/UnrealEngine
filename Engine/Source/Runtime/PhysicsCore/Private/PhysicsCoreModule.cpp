// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsCoreModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE( IPhysicsCoreModule, PhysicsCore );
