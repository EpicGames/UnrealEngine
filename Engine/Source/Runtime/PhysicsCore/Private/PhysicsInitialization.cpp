// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsInitialization.h"
#include "PhysXPublicCore.h"
#include "PhysicsPublicCore.h"
#include "PhysXSupportCore.h"
#include "Misc/CommandLine.h"

#include "Modules/ModuleManager.h"
#include "Misc/Paths.h"
#include "HAL/IConsoleManager.h"

#ifndef APEX_STATICALLY_LINKED
#define APEX_STATICALLY_LINKED	0
#endif

// CVars
TAutoConsoleVariable<float> CVarToleranceScaleLength(
	TEXT("p.ToleranceScale_Length"),
	100.f,
	TEXT("The approximate size of objects in the simulation. Default: 100"),
	ECVF_ReadOnly);

TAutoConsoleVariable<float> CVarToleranceScaleSpeed(
	TEXT("p.ToleranceScale_Speed"),
	1000.f,
	TEXT("The typical magnitude of velocities of objects in simulation. Default: 1000"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarUseUnifiedHeightfield(
	TEXT("p.bUseUnifiedHeightfield"),
	1,
	TEXT("Whether to use the PhysX unified heightfield. This feature of PhysX makes landscape collision consistent with triangle meshes but the thickness parameter is not supported for unified heightfields. 1 enables and 0 disables. Default: 1"),
	ECVF_ReadOnly);

bool InitGamePhysCore()
{
	// If we're running with Chaos enabled, load its module
	FModuleManager::Get().LoadModule("Chaos");

#if WITH_ENGINE
	// Loading this without Chaos gives warning, as no module depends on it.
	FModuleManager::Get().LoadModule("ChaosSolverEngine");
#endif

	return true;
}

void TermGamePhysCore()
{

}
