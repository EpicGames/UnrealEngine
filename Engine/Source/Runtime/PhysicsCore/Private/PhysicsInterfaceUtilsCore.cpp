// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsInterfaceUtilsCore.h"
#include "PhysXPublicCore.h"
#include "PhysicsInterfaceTypesCore.h"

#include "Chaos/ParticleHandle.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FCollisionFilterData C2UFilterData(const FChaosFilterData& PFilterData)
{
	FCollisionFilterData FilterData;
	FilterData.Word0 = PFilterData.word0;
	FilterData.Word1 = PFilterData.word1;
	FilterData.Word2 = PFilterData.word2;
	FilterData.Word3 = PFilterData.word3;
	return FilterData;
}

FChaosFilterData U2CFilterData(const FCollisionFilterData& FilterData)
{
	return FChaosFilterData(FilterData.Word0, FilterData.Word1, FilterData.Word2, FilterData.Word3);
}

FCollisionFilterData ToUnrealFilterData(const FChaosFilterData& FilterData)
{
	return C2UFilterData(FilterData);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

template <typename THitLocation>
uint32 FindFaceIndexImp(const THitLocation& PHit, const FVector& UnitDir)
{
	const FTransform WorldTM(PHit.Actor->GetR(), PHit.Actor->GetX());
	const FVector LocalPosition = WorldTM.InverseTransformPositionNoScale(PHit.WorldPosition);
	const FVector LocalUnitDir = WorldTM.InverseTransformVectorNoScale(UnitDir);
	return PHit.Shape->GetGeometry()->FindMostOpposingFace(LocalPosition, LocalUnitDir, PHit.FaceIndex, 1);	//todo:this number matches the one above, but is it right?
}

uint32 FindFaceIndex(const FHitLocation& Hit, const FVector& UnitDir)
{
	return FindFaceIndexImp(Hit, UnitDir);
}

uint32 FindFaceIndex(const ChaosInterface::FPTLocationHit& Hit, const FVector& UnitDir)
{
	return FindFaceIndexImp(Hit, UnitDir);
}
