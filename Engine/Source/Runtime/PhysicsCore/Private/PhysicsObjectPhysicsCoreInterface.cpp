// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsObjectPhysicsCoreInterface.h"

#include "Chaos/PhysicsObjectInterface.h"
#include "PBDRigidsSolver.h"

FChaosScene* PhysicsObjectPhysicsCoreInterface::GetScene(TArrayView<const Chaos::FConstPhysicsObjectHandle> InObjects)
{
	Chaos::FPBDRigidsSolver* Solver = Chaos::FPhysicsObjectInterface::GetSolver(InObjects);
	if (!Solver)
	{
		return nullptr;
	}
	return reinterpret_cast<FChaosScene*>(Solver->PhysSceneHack);
}

FChaosScene* PhysicsObjectPhysicsCoreInterface::GetScene(const Chaos::FConstPhysicsObjectHandle InObject)
{
	Chaos::FPBDRigidsSolver* Solver = Chaos::FPhysicsObjectInterface::GetSolver(InObject);
	if (!Solver)
	{
		return nullptr;
	}
	return reinterpret_cast<FChaosScene*>(Solver->PhysSceneHack);
}
