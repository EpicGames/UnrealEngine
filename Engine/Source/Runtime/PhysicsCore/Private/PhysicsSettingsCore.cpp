// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsSettingsCore.h"
#include "Chaos/ChaosEngineInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsSettingsCore)

UPhysicsSettingsCore* UPhysicsSettingsCore::DefaultSettings = nullptr;

void UPhysicsSettingsCore::SetDefaultSettings(UPhysicsSettingsCore* InSettings)
{
	DefaultSettings = InSettings;
}

UPhysicsSettingsCore* UPhysicsSettingsCore::Get()
{
	// Use the override if we have it.
	if (DefaultSettings != nullptr)
	{
		return DefaultSettings;
	}

	// Use CDO otherwise
	return CastChecked<UPhysicsSettingsCore>(UPhysicsSettingsCore::StaticClass()->GetDefaultObject());
}

UPhysicsSettingsCore::UPhysicsSettingsCore(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultGravityZ(-980.f)
	, DefaultTerminalVelocity(4000.f)
	, DefaultFluidFriction(0.3f)
	, SimulateScratchMemorySize(262144)
	, RagdollAggregateThreshold(4)
	, TriangleMeshTriangleMinAreaThreshold(5.0f)
	, bEnableEnhancedDeterminism(false)
	, bEnableShapeSharing(false)
	, bEnablePCM(true)
	, bEnableStabilization(false)
	, bWarnMissingLocks(true)
	, bEnable2DPhysics(false)
	, bDefaultHasComplexCollision_DEPRECATED(true)
	, BounceThresholdVelocity(200.f)
	, MaxAngularVelocity(3600)	//10 revolutions per second
	, ContactOffsetMultiplier(0.02f)
	, MinContactOffset(2.f)
	, MaxContactOffset(8.f)
	, bSimulateSkeletalMeshOnDedicatedServer(true)
	, DefaultShapeComplexity((ECollisionTraceFlag)-1)
{
	SectionName = TEXT("Physics");
}

void UPhysicsSettingsCore::PostInitProperties()
{
	Super::PostInitProperties();

	if(DefaultShapeComplexity == TEnumAsByte<ECollisionTraceFlag>(-1))
	{
		DefaultShapeComplexity = bDefaultHasComplexCollision_DEPRECATED ? CTF_UseSimpleAndComplex : CTF_UseSimpleAsComplex;
	}

	SolverOptions.MoveRenamedPropertyValues();
}


