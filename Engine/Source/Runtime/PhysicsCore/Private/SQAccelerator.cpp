// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQAccelerator.h"
#include "SQVisitor.h"
#include "CollisionQueryFilterCallbackCore.h"
#include "PhysicsInterfaceUtilsCore.h"

#include "SceneQueryChaosImp.h"

#include "ChaosInterfaceWrapperCore.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/DebugDrawQueue.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FSQAcceleratorUnion::Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, FPhysicsHitCallback<FHitRaycast>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const
{
	for(const ISQAccelerator* Accelerator : Accelerators)
	{
		Accelerator->Raycast(Start, Dir, DeltaMagnitude, HitBuffer, OutputFlags, QueryFilterData, QueryCallback);
	}
}

void FSQAcceleratorUnion::Sweep(const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, FPhysicsHitCallback<FHitSweep>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const
{
	for(const ISQAccelerator* Accelerator : Accelerators)
	{
		Accelerator->Sweep(QueryGeom, StartTM, Dir, DeltaMagnitude, HitBuffer, OutputFlags, QueryFilterData, QueryCallback);
	}
}

void FSQAcceleratorUnion::Overlap(const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<FHitOverlap>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const
{
	for(const ISQAccelerator* Accelerator : Accelerators)
	{
		Accelerator->Overlap(QueryGeom, GeomPose, HitBuffer, QueryFilterData, QueryCallback);
	}
}

void FSQAcceleratorUnion::AddSQAccelerator(ISQAccelerator* InAccelerator)
{
	Accelerators.AddUnique(InAccelerator);
}

void FSQAcceleratorUnion::RemoveSQAccelerator(ISQAccelerator* AcceleratorToRemove)
{
	Accelerators.RemoveSingleSwap(AcceleratorToRemove);	//todo(ocohen): probably want to order these in some optimal way
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FChaosSQAccelerator::FChaosSQAccelerator(const Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>& InSpatialAcceleration)
	: SpatialAcceleration(InSpatialAcceleration)
{}

struct FPreFilterInfo
{
	const Chaos::FImplicitObject* Geom;
	int32 ActorIdx;
};

template <typename TRaycastHit>
void FChaosSQAccelerator::RaycastImp(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<TRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
{
	using namespace Chaos;
	using namespace ChaosInterface;

	TSQVisitor<TSphere<FReal, 3>, FAccelerationStructureHandle, TRaycastHit, std::is_same<TRaycastHit, FRaycastHit>::value> RaycastVisitor(Start, Dir, HitBuffer, OutputFlags, CommonParams);
	HitBuffer.IncFlushCount();
	SpatialAcceleration.Raycast(Start, Dir, DeltaMagnitude, RaycastVisitor);
	HitBuffer.DecFlushCount();
}

void FChaosSQAccelerator::Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
{
	RaycastImp(Start, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonParams);
}

void FChaosSQAccelerator::Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
{
	RaycastImp(Start, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonParams);
}

void FChaosSQAccelerator::Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, StartTM, [this, &Dir, &DeltaMagnitude, &HitBuffer, &OutputFlags, &CommonParams](const auto& Downcast, const FTransform& StartFullTM)
	{
		return SweepHelper(Downcast, SpatialAcceleration, StartFullTM, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonParams);
	});
}

void FChaosSQAccelerator::Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, StartTM, [this, &Dir, &DeltaMagnitude, &HitBuffer, &OutputFlags, &CommonParams](const auto& Downcast, const FTransform& StartFullTM)
	{
		return SweepHelper(Downcast, SpatialAcceleration, StartFullTM, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonParams);
	});
}

void FChaosSQAccelerator::Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, GeomPose, [this, &HitBuffer, &CommonParams](const auto& Downcast, const FTransform& GeomFullPose)
	{
		return OverlapHelper(Downcast, SpatialAcceleration, GeomFullPose, HitBuffer, CommonParams);
	});
}

void FChaosSQAccelerator::Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTOverlapHit>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, GeomPose, [this, &HitBuffer, &CommonParams](const auto& Downcast, const FTransform& GeomFullPose)
	{
		return OverlapHelper(Downcast, SpatialAcceleration, GeomFullPose, HitBuffer, CommonParams);
	});
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FChaosSQAccelerator::Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) const
{
	const ChaosInterface::FSceneQueryCommonParams CommonData(QueryCallback, QueryFilterData, DebugParams);
	RaycastImp(Start, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonData);
}

void FChaosSQAccelerator::Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) const
{
	const ChaosInterface::FSceneQueryCommonParams CommonData(QueryCallback, QueryFilterData, DebugParams);
	RaycastImp(Start, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonData);
}

void FChaosSQAccelerator::Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, StartTM, [&](const auto& Downcast, const FTransform& StartFullTM)
	{
		return SweepHelper(Downcast, SpatialAcceleration, StartFullTM, Dir, DeltaMagnitude, HitBuffer, OutputFlags, QueryFilterData, QueryCallback, DebugParams);
	});
}

void FChaosSQAccelerator::Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, StartTM, [&](const auto& Downcast, const FTransform& StartFullTM)
	{
		return SweepHelper(Downcast, SpatialAcceleration, StartFullTM, Dir, DeltaMagnitude, HitBuffer, OutputFlags, QueryFilterData, QueryCallback, DebugParams);
	});
}

void FChaosSQAccelerator::Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, GeomPose, [&](const auto& Downcast, const FTransform& GeomFullPose)
	{
		return OverlapHelper(Downcast, SpatialAcceleration, GeomFullPose, HitBuffer, QueryFilterData, QueryCallback, DebugParams);
	});
}

void FChaosSQAccelerator::Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTOverlapHit>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams) const
{
	return Chaos::Utilities::CastHelper(QueryGeom, GeomPose, [&](const auto& Downcast, const FTransform& GeomFullPose) { return OverlapHelper(Downcast, SpatialAcceleration, GeomFullPose, HitBuffer, QueryFilterData, QueryCallback, DebugParams); });
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
