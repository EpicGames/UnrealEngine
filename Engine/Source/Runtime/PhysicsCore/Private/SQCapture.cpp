// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQCapture.h"

#include "Chaos/ImplicitObject.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "PhysicsInterfaceTypesCore.h"

#include "PhysXSupportCore.h"
#include "PhysicsPublicCore.h"
#include "PhysTestSerializer.h" 

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FSQCaptureFilterCallback : public ICollisionQueryFilterCallbackBase
{
public:
	FSQCaptureFilterCallback(const FSQCapture& InCapture) : Capture(InCapture) {}
	virtual ~FSQCaptureFilterCallback() {}
	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) override { /*check(false);*/  return ECollisionQueryHitType::Touch; }
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) override { return Capture.GetFilterResult(&Shape, &Actor); }

	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) override { ensure(false);  return ECollisionQueryHitType::Touch; }
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) override { ensure(false);  return ECollisionQueryHitType::Touch; }

private:
	const FSQCapture& Capture;
};

FSQCapture::FSQCapture(FPhysTestSerializer& InPhysSerializer)
	: OutputFlags(EHitFlags::None)
	, ChaosImplicitGeometry(nullptr)
	, PhysSerializer(InPhysSerializer)
	, bDiskDataIsChaos(false)
	, bChaosDataReady(false)
	, bPhysXDataReady(false)
{
}

FSQCapture::~FSQCapture()
{
}

void SerializeQueryFilterData(FArchive& Ar, ChaosInterface::FQueryFilterData& QueryFilterData)
{
	Ar << QueryFilterData.data.word0;
	Ar << QueryFilterData.data.word1;
	Ar << QueryFilterData.data.word2;
	Ar << QueryFilterData.data.word3;
	uint16 Flags = QueryFilterData.flags;
	Ar << Flags;
	QueryFilterData.flags = (FChaosQueryFlags)Flags;
	Ar << QueryFilterData.clientId;
}

void FSQCapture::SerializeChaosActorToShapeHitsArray(Chaos::FChaosArchive& Ar)
{
	int32 NumActors = ChaosActorToShapeHitsArray.Num();
	Ar << NumActors;
	if (Ar.IsLoading())
	{
		for (int32 ActorIdx = 0; ActorIdx < NumActors; ++ActorIdx)
		{
			Chaos::TSerializablePtr<Chaos::FGeometryParticle>Actor;
			Ar << Actor;
			int32 NumShapes;
			Ar << NumShapes;

			TArray<TPair<Chaos::FPerShapeData*, ECollisionQueryHitType>> Pairs;
			for (int32 ShapeIdx = 0; ShapeIdx < NumShapes; ++ShapeIdx)
			{
				Chaos::TSerializablePtr<Chaos::FPerShapeData> Shape;
				Ar << Shape;
				ECollisionQueryHitType HitType;
				Ar << HitType;
				Pairs.Emplace(const_cast<Chaos::FPerShapeData*>(Shape.Get()), HitType);
			}
			ChaosActorToShapeHitsArray.Add(const_cast<Chaos::FGeometryParticle*>(Actor.Get()), Pairs);
		}
	}
	else if (Ar.IsSaving())
	{
		for (auto& Itr : ChaosActorToShapeHitsArray)
		{
			Ar << AsAlwaysSerializable(Itr.Key);
			int32 NumShapes = Itr.Value.Num();
			Ar << NumShapes;

			for (auto& Pair : Itr.Value)
			{
				Ar << AsAlwaysSerializable(Pair.Key);
				Ar << Pair.Value;

			}
		}
	}
}

template <typename THit>
void FSQCapture::SerializeChaosBuffers(Chaos::FChaosArchive& Ar, int32 Version, ChaosInterface::FSQHitBuffer<THit>& ChaosBuffer)
{
	bool bHasBlock = ChaosBuffer.HasBlockingHit();
	Ar << bHasBlock;

	if (bHasBlock)
	{
		if (Ar.IsLoading())
		{
			THit Hit;
			Ar << Hit;
			ChaosBuffer.SetBlockingHit(Hit);
		}
		else
		{
			THit Hit = *ChaosBuffer.GetBlock();
			Ar << Hit;
		}
	}

	int32 NumHits;
	
	NumHits = ChaosBuffer.GetNumHits();
	Ar << NumHits;

	for (int32 Idx = 0; Idx < NumHits; ++Idx)
	{
		if (Ar.IsLoading())
		{
			THit Touch;
			Ar << Touch;
			ChaosBuffer.AddTouchingHit(Touch);
		}
		else
		{
			THit* Hits = ChaosBuffer.GetHits();
			Ar << Hits[Idx];
		}
	}
	
}

void FSQCapture::Serialize(Chaos::FChaosArchive& Ar)
{
	static const FName SQCaptureName = "SQCapture";
	Chaos::FChaosArchiveScopedMemory ScopedMemory(Ar, SQCaptureName, false);

	int32 Version = 2;
	Ar << Version;
	Ar << SQType;
	Ar << bDiskDataIsChaos;
	Ar << Dir << StartTM << DeltaMag << OutputFlags;
	Ar << GeomData;
	Ar << HitData;

	if (Version >= 1)
	{
		Ar << StartPoint;
	}

	if (bDiskDataIsChaos == false)
	{
		if(Version >= 1)
		{
			SerializeQueryFilterData(Ar, QueryFilterData);
		}
	}

	if (bDiskDataIsChaos)
	{
		SerializeChaosBuffers(Ar, Version, ChaosSweepBuffer);
		SerializeChaosBuffers(Ar, Version, ChaosRaycastBuffer);
		SerializeChaosBuffers(Ar, Version, ChaosOverlapBuffer);

		SerializeChaosActorToShapeHitsArray(Ar);
		SerializeQueryFilterData(Ar, QueryFilterData);

		if (Version >= 2)
		{
			Ar << ChaosImplicitGeometry;
		}
	}

	if (Ar.IsLoading())
	{
		if (bDiskDataIsChaos)
		{
			FilterCallback = MakeUnique<FSQCaptureFilterCallback>(*this);
		}
	}
}

void FSQCapture::StartCaptureChaosSweep(const Chaos::FPBDRigidsEvolution& Evolution, const Chaos::FImplicitObject& InQueryGeom, const FTransform& InStartTM, const FVector& InDir, float InDeltaMag, FHitFlags InOutputFlags, const ChaosInterface::FQueryFilterData& QueryFilter, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback)
{
	if (IsInGameThread())
	{
		bDiskDataIsChaos = true;
		CaptureChaosFilterResults(Evolution, FilterData, Callback);
		//copy data
		ChaosImplicitGeometry = InQueryGeom.CopyGeometry();
		StartTM = InStartTM;
		Dir = InDir;
		DeltaMag = InDeltaMag;
		OutputFlags = InOutputFlags;
		QueryFilterData = QueryFilter;

		SQType = ESQType::Sweep;
	}
}

void FSQCapture::EndCaptureChaosSweep(const ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>& Results)
{
	if (IsInGameThread())
	{
		check(SQType == ESQType::Sweep);
		ChaosSweepBuffer = Results;
	}
}

void FSQCapture::StartCaptureChaosRaycast(const Chaos::FPBDRigidsEvolution& Evolution, const FVector& InStartPoint, const FVector& InDir, float InDeltaMag, FHitFlags InOutputFlags, const ChaosInterface::FQueryFilterData& QueryFilter, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback)
{
	if (IsInGameThread())
	{
		bDiskDataIsChaos = true;
		CaptureChaosFilterResults(Evolution, FilterData, Callback);
		//copy data
		StartPoint = InStartPoint;
		Dir = InDir;
		DeltaMag = InDeltaMag;
		OutputFlags = InOutputFlags;
		QueryFilterData = QueryFilter;

		SQType = ESQType::Raycast;
	}
}

void FSQCapture::EndCaptureChaosRaycast(const ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>& Results)
{
	if (IsInGameThread())
	{
		check(SQType == ESQType::Raycast);
		ChaosRaycastBuffer = Results;
	}
}

void FSQCapture::StartCaptureChaosOverlap(const Chaos::FPBDRigidsEvolution& Evolution, const Chaos::FImplicitObject& InQueryGeom, const FTransform& InStartTM, const ChaosInterface::FQueryFilterData& QueryFilter, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback)
{
	if (IsInGameThread())
	{
		bDiskDataIsChaos = true;
		CaptureChaosFilterResults(Evolution, FilterData, Callback);
		//copy data
		ChaosImplicitGeometry = InQueryGeom.CopyGeometry();
		StartTM = InStartTM;
		QueryFilterData = QueryFilter;

		SQType = ESQType::Overlap;
	}
}

void FSQCapture::EndCaptureChaosOverlap(const ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>& Results)
{
	if (IsInGameThread())
	{
		check(SQType == ESQType::Overlap);
		ChaosOverlapBuffer = Results;
	}
}

void FSQCapture::CaptureChaosFilterResults(const Chaos::FPBDRigidsEvolution& TransientEvolution, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback)
{
	using namespace Chaos;
	const FPBDRigidsSOAs& Particles = TransientEvolution.GetParticles();
	const int32 NumTransientActors = Particles.GetParticleHandles().Size();

	for (int32 Idx = 0; Idx < NumTransientActors; ++Idx)
	{
		FGeometryParticle* TransientActor = Particles.GetParticleHandles().Handle(Idx)->GTGeometryParticle();
		const FShapesArray& TransientShapes = TransientActor->ShapesArray();
		const int32 NumTransientShapes = TransientShapes.Num();
		TArray<TPair<FPerShapeData*, ECollisionQueryHitType>> ShapeHitsArray; ShapeHitsArray.Reserve(NumTransientShapes);
		for (const auto& TransientShape : TransientShapes)
		{
			const ECollisionQueryHitType Result = Callback.PreFilter(FilterData, *TransientShape, *TransientActor);
			ShapeHitsArray.Emplace(TransientShape.Get(), Result);
		}

		ChaosActorToShapeHitsArray.Add(TransientActor, ShapeHitsArray);
	}
}

template <typename TShape, typename TActor>
ECollisionQueryHitType GetFilterResultHelper(const TShape* Shape, const TActor* Actor, const TMap<TActor*, TArray<TPair<TShape*, ECollisionQueryHitType>>>& ActorToShapeHitsArray)
{
	if (const TArray<TPair<TShape*, ECollisionQueryHitType>>* ActorToPairs = ActorToShapeHitsArray.Find(Actor))
	{
		for (const TPair<TShape*, ECollisionQueryHitType>& Pair : *ActorToPairs)
		{
			if (Pair.Key == Shape)
			{
				return Pair.Value;
			}
		}
	}

	//todo: figure out why this hits - suspect it's related to threading and how we capture an evolution on GT
	ensure(false);	//should not get here, means we didn't properly capture all filter results
	return ECollisionQueryHitType::None;
}

ECollisionQueryHitType FSQCapture::GetFilterResult(const Chaos::FPerShapeData* Shape, const Chaos::FGeometryParticle* Actor) const
{
	return GetFilterResultHelper(Shape, Actor, ChaosActorToShapeHitsArray);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
