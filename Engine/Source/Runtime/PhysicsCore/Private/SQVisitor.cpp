// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQVisitor.h"

#if CHAOS_DEBUG_DRAW
int32 ChaosSQDrawDebugVisitorQueries = 0;
FAutoConsoleVariableRef CVarChaosSQDrawDebugQueries(TEXT("p.Chaos.SQ.DrawDebugVisitorQueries"), ChaosSQDrawDebugVisitorQueries, TEXT("Draw bounds of objects visited by visitors in scene queries."));
#endif