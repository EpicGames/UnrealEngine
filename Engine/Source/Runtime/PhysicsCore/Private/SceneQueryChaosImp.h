// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "PhysicsInterfaceDeclaresCore.h"

enum class EHitFlags : uint16;

inline bool LowLevelRaycastImp(const FVector& Start, const FVector& Dir, float DeltaMag, const Chaos::FImplicitObject& Shape, const FTransform ActorTM, EHitFlags OutputFlags, FHitRaycast& Hit)
{
	//TODO_SQ_IMPLEMENTATION
	return false;			
}

inline bool LowLevelSweepImp(const FTransform& StartTM, const FVector& Dir, float DeltaMag, const FPhysicsGeometry& SweepGeom, const Chaos::FImplicitObject& Shape, const FTransform ActorTM, EHitFlags OutputFlags, FHitSweep& Hit)
{
	//TODO_SQ_IMPLEMENTATION
	return false;
}

inline bool LowLevelOverlapImp(const FTransform& GeomPose, const FPhysicsGeometry& OverlapGeom, const Chaos::FImplicitObject& Shape, const FTransform ActorTM, FHitOverlap& Overlap)
{
	//TODO_SQ_IMPLEMENTATION
	return false;
}