// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneQueryCommonParams.h"

namespace ChaosInterface
{
	FSceneQueryCommonParams::FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback)
		: QueryCallback(&InQueryCallback)
	{
	}

	FSceneQueryCommonParams::FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback, const FQueryFilterData& InFilterData, FChaosQueryFlags InFlags, const FQueryDebugParams& InDebugParams)
		: FilterData(InFilterData)
		, Flags(InFlags)
		, DebugParams(InDebugParams)
		, QueryCallback(&InQueryCallback)
	{
	}

	FSceneQueryCommonParams::FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback, const FQueryFilterData& InFilterData, EQueryFlags InFlags, const FQueryDebugParams& InDebugParams)
		: FSceneQueryCommonParams(InQueryCallback, InFilterData, U2CQueryFlags(InFlags), InDebugParams)
	{
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSceneQueryCommonParams::FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback, const FChaosQueryFilterData& InFilterData, const FQueryDebugParams& InDebugParams)
		: DebugParams(InDebugParams)
		, QueryCallback(&InQueryCallback)
	{
		FCollisionFilterData CollisionFilterData;
		CollisionFilterData.Word0 = InFilterData.data.word0;
		CollisionFilterData.Word1 = InFilterData.data.word1;
		CollisionFilterData.Word2 = InFilterData.data.word2;
		CollisionFilterData.Word3 = InFilterData.data.word3;

		FilterData = Chaos::Filter::FQueryFilterBuilder::BuildFromLegacyQueryFilter(CollisionFilterData);
		Flags = InFilterData.flags;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ICollisionQueryFilterCallback& FSceneQueryCommonParams::GetQueryCallback() const
	{
		return *QueryCallback;
	}
} // namespace ChaosInterface
