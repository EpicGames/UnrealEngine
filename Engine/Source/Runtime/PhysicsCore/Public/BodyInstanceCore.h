// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "BodyInstanceCore.generated.h"

class UBodySetupCore;

USTRUCT(BlueprintType)
struct FBodyInstanceCore
{
	GENERATED_USTRUCT_BODY()

	/** BodySetupCore pointer that this instance is initialized from */
	TWeakObjectPtr<UBodySetupCore> BodySetup;

	PHYSICSCORE_API FBodyInstanceCore();

	/** 
	 * If true, this body will use simulation. If false, will be 'fixed' (ie kinematic) and move where it is told. 
	 * For a Skeletal Mesh Component, simulating requires a physics asset setup and assigned on the SkeletalMesh asset.
	 * For a Static Mesh Component, simulating requires simple collision to be setup on the StaticMesh asset.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics)
	uint8 bSimulatePhysics : 1;


	/** If true, mass will not be automatically computed and you must set it directly */
	UPROPERTY(EditAnywhere,Category = Physics,meta = (InlineEditConditionToggle))
	uint8 bOverrideMass : 1;

	/** If object should have the force of gravity applied */
	UPROPERTY(EditAnywhere,BlueprintReadOnly,Category = Physics)
	uint8 bEnableGravity : 1;

	/** When kinematic, whether the actor transform should be updated as a result of movement in the simulation, rather than immediately whenever a target transform is set. */
	UPROPERTY(EditAnywhere,BlueprintReadOnly,Category = Physics)
	uint8 bUpdateKinematicFromSimulation : 1;

	/** Enabled/disables whether this body is affected by gyroscopic torque, mainly useful for long/thin objects that spin */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Physics)
	uint8 bGyroscopicTorqueEnabled : 1;

	/** If true and is attached to a parent, the two bodies will be joined into a single rigid body. Physical settings like collision profile and body settings are determined by the root */
	UPROPERTY(EditAnywhere,AdvancedDisplay,BlueprintReadWrite,Category = Physics,meta = (editcondition = "!bSimulatePhysics"))
	uint8 bAutoWeld : 1;

	/** If object should start awake, or if it should initially be sleeping */
	UPROPERTY(EditAnywhere,AdvancedDisplay,BlueprintReadOnly,Category = Physics,meta = (editcondition = "bSimulatePhysics"))
	uint8 bStartAwake:1;

	/**	Should 'wake/sleep' events fire when this object is woken up or put to sleep by the physics simulation. */
	UPROPERTY(EditAnywhere,AdvancedDisplay,BlueprintReadOnly,Category = Physics)
	uint8 bGenerateWakeEvents : 1;

	/** If true, it will update mass when scale change **/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=Physics)
	uint8 bUpdateMassWhenScaleChanges:1;

	/** Indicates mass props need to be recomputed when switching from kinematic to simulated*/
	uint8 bDirtyMassProps : 1;

	/** Should Simulate Physics **/
	PHYSICSCORE_API bool ShouldInstanceSimulatingPhysics() const;
};