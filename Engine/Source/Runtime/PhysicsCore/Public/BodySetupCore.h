// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "BodySetupEnums.h"

#include "BodySetupCore.generated.h"

namespace physx
{
	class PxTriangleMesh;
}

UCLASS(collapseCategories, MinimalAPI)
class UBodySetupCore : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	PHYSICSCORE_API TEnumAsByte<enum ECollisionTraceFlag> GetCollisionTraceFlag() const;

	/** Used in the PhysicsAsset case. Associates this Body with Bone in a skeletal mesh. */
	UPROPERTY(Category=BodySetup,VisibleAnywhere)
	FName BoneName;
	
	/** 
	 *	If simulated it will use physics, if kinematic it will not be affected by physics, but can interact with physically simulated bodies. Default will inherit from OwnerComponent's behavior.
	 */
	UPROPERTY(EditAnywhere, Category=Physics)
	TEnumAsByte<EPhysicsType> PhysicsType;

	/** Collision Trace behavior - by default, it will keep simple(convex)/complex(per-poly) separate **/
	UPROPERTY(EditAnywhere,Category=Collision,meta=(DisplayName = "Collision Complexity"))
	TEnumAsByte<enum ECollisionTraceFlag> CollisionTraceFlag;

	/** Collision Type for this body. This eventually changes response to collision to others **/
	UPROPERTY(EditAnywhere,Category=Collision)
	TEnumAsByte<enum EBodyCollisionResponse::Type> CollisionReponse;
};
