// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BodySetupEnums.generated.h"

UENUM(BlueprintType)
enum ECollisionTraceFlag : int
{
	/** Use project physics settings (DefaultShapeComplexity) */
	CTF_UseDefault UMETA(DisplayName="Project Default"),
	/** Create both simple and complex shapes. Simple shapes are used for regular scene queries and collision tests. Complex shape (per poly) is used for complex scene queries.*/
	CTF_UseSimpleAndComplex UMETA(DisplayName = "Simple And Complex"),
	/** Create only simple shapes. Use simple shapes for all scene queries and collision tests.*/
	CTF_UseSimpleAsComplex UMETA(DisplayName = "Use Simple Collision As Complex"),
	/** Create only complex shapes (per poly). Use complex shapes for all scene queries and collision tests. Can be used in simulation for static shapes only (i.e can be collided against but not moved through forces or velocity.) */
	CTF_UseComplexAsSimple UMETA(DisplayName = "Use Complex Collision As Simple"),
	CTF_MAX,
};

UENUM()
enum EPhysicsType : int
{
	/** Follow owner. */
	PhysType_Default UMETA(DisplayName="Default"),	
	/** Do not follow owner, but make kinematic. */
	PhysType_Kinematic	UMETA(DisplayName="Kinematic"),		
	/** Do not follow owner, but simulate. */
	PhysType_Simulated	UMETA(DisplayName="Simulated")	
};

UENUM()
namespace EBodyCollisionResponse
{
	enum Type : int
	{
		BodyCollision_Enabled UMETA(DisplayName="Enabled"), 
		BodyCollision_Disabled UMETA(DisplayName="Disabled")//, 
		//BodyCollision_Custom UMETA(DisplayName="Custom")
	};
}


/** Helpers to convert enum to string */
PHYSICSCORE_API const TCHAR* LexToString(ECollisionTraceFlag Enum);
PHYSICSCORE_API const TCHAR* LexToString(EPhysicsType Enum);
PHYSICSCORE_API const TCHAR* LexToString(EBodyCollisionResponse::Type Enum);
