// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/Real.h"

namespace Chaos
{
	/**
	*  Constraint settings all for generic defaults and scalars to the constraint parameters so
	*  that converting between existing (pre v4.26) simulation parameters and Chaos can produce similar 
	*  simulation results. 
	*/
	class ConstraintSettings
	{
		ConstraintSettings(){};

	public:
		static PHYSICSCORE_API FReal JointStiffness();
		static PHYSICSCORE_API FReal LinearDriveStiffnessScale();
		static PHYSICSCORE_API FReal LinearDriveDampingScale();
		static PHYSICSCORE_API FReal AngularDriveStiffnessScale();
		static PHYSICSCORE_API FReal AngularDriveDampingScale();

		static PHYSICSCORE_API int SoftLinearForceMode();
		static PHYSICSCORE_API FReal SoftLinearStiffnessScale();
		static PHYSICSCORE_API FReal SoftLinearDampingScale();

		static PHYSICSCORE_API int SoftAngularForceMode();
		static PHYSICSCORE_API FReal SoftAngularStiffnessScale();
		static PHYSICSCORE_API FReal SoftAngularDampingScale();

		static PHYSICSCORE_API FReal LinearBreakScale();
		static PHYSICSCORE_API FReal AngularBreakScale();
	};
}
