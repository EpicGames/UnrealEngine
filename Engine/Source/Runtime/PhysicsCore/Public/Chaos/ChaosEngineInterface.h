// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/Declares.h"
#include "Chaos/PhysicsObject.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceWrapperShared.h"
#include "PhysicsInterfaceTypesCore.h"
#include "Chaos/Evolution/IterationSettings.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "ChaosEngineInterface.generated.h"

//NOTE: Do not include Chaos headers directly as it means recompiling all of engine. This should be reworked to avoid allocations


/** Types of surfaces in the game, used by Physical Materials */
UENUM(BlueprintType)
enum EPhysicalSurface : int
{
	SurfaceType_Default UMETA(DisplayName="Default"),
	SurfaceType1 UMETA(Hidden),
	SurfaceType2 UMETA(Hidden),
	SurfaceType3 UMETA(Hidden),
	SurfaceType4 UMETA(Hidden),
	SurfaceType5 UMETA(Hidden),
	SurfaceType6 UMETA(Hidden),
	SurfaceType7 UMETA(Hidden),
	SurfaceType8 UMETA(Hidden),
	SurfaceType9 UMETA(Hidden),
	SurfaceType10 UMETA(Hidden),
	SurfaceType11 UMETA(Hidden),
	SurfaceType12 UMETA(Hidden),
	SurfaceType13 UMETA(Hidden),
	SurfaceType14 UMETA(Hidden),
	SurfaceType15 UMETA(Hidden),
	SurfaceType16 UMETA(Hidden),
	SurfaceType17 UMETA(Hidden),
	SurfaceType18 UMETA(Hidden),
	SurfaceType19 UMETA(Hidden),
	SurfaceType20 UMETA(Hidden),
	SurfaceType21 UMETA(Hidden),
	SurfaceType22 UMETA(Hidden),
	SurfaceType23 UMETA(Hidden),
	SurfaceType24 UMETA(Hidden),
	SurfaceType25 UMETA(Hidden),
	SurfaceType26 UMETA(Hidden),
	SurfaceType27 UMETA(Hidden),
	SurfaceType28 UMETA(Hidden),
	SurfaceType29 UMETA(Hidden),
	SurfaceType30 UMETA(Hidden),
	SurfaceType31 UMETA(Hidden),
	SurfaceType32 UMETA(Hidden),
	SurfaceType33 UMETA(Hidden),
	SurfaceType34 UMETA(Hidden),
	SurfaceType35 UMETA(Hidden),
	SurfaceType36 UMETA(Hidden),
	SurfaceType37 UMETA(Hidden),
	SurfaceType38 UMETA(Hidden),
	SurfaceType39 UMETA(Hidden),
	SurfaceType40 UMETA(Hidden),
	SurfaceType41 UMETA(Hidden),
	SurfaceType42 UMETA(Hidden),
	SurfaceType43 UMETA(Hidden),
	SurfaceType44 UMETA(Hidden),
	SurfaceType45 UMETA(Hidden),
	SurfaceType46 UMETA(Hidden),
	SurfaceType47 UMETA(Hidden),
	SurfaceType48 UMETA(Hidden),
	SurfaceType49 UMETA(Hidden),
	SurfaceType50 UMETA(Hidden),
	SurfaceType51 UMETA(Hidden),
	SurfaceType52 UMETA(Hidden),
	SurfaceType53 UMETA(Hidden),
	SurfaceType54 UMETA(Hidden),
	SurfaceType55 UMETA(Hidden),
	SurfaceType56 UMETA(Hidden),
	SurfaceType57 UMETA(Hidden),
	SurfaceType58 UMETA(Hidden),
	SurfaceType59 UMETA(Hidden),
	SurfaceType60 UMETA(Hidden),
	SurfaceType61 UMETA(Hidden),
	SurfaceType62 UMETA(Hidden),
	SurfaceType_Max UMETA(Hidden)
};

/** Enum for controlling the falloff of strength of a radial impulse as a function of distance from Origin. */
UENUM()
enum ERadialImpulseFalloff : int
{
	/** Impulse is a constant strength, up to the limit of its range. */
	RIF_Constant,
	/** Impulse should get linearly weaker the further from origin. */
	RIF_Linear,
	RIF_MAX,
};

/** Presets of values used in considering when put this body to sleep. */
UENUM()
enum class ESleepFamily: uint8
{
	/** Engine defaults. */
	Normal,
	/** A family of values with a lower sleep threshold; good for slower pendulum-like physics. */
	Sensitive,
	/** Specify your own sleep threshold multiplier */
	Custom,
};

/** Specifies angular degrees of freedom */
UENUM()
enum EAngularConstraintMotion : int
{
	/** No constraint against this axis. */
	ACM_Free		UMETA(DisplayName="Free"),
	/** Limited freedom along this axis. */
	ACM_Limited		UMETA(DisplayName="Limited"),
	/** Fully constraint against this axis. */
	ACM_Locked		UMETA(DisplayName="Locked"),

	ACM_MAX,
};

/** Enum to indicate which context frame we use for physical constraints */
UENUM()
namespace EConstraintFrame
{
enum Type : int
{
	Frame1,
	Frame2
};
}


namespace PhysicsInterfaceTypes
{
	enum class ELimitAxis : uint8
	{
		X,
		Y,
		Z,
		Twist,
		Swing1,
		Swing2
	};

	enum class EDriveType : uint8
	{
		X,
		Y,
		Z,
		Swing,
		Twist,
		Slerp
	};

	/**
	* Default number of inlined elements used in FInlineShapeArray.
	* Increase if for instance character meshes use more than this number of physics bodies and are involved in many queries.
	*/
	enum { NumInlinedPxShapeElements = 32 };

	/** Array that is intended for use when fetching shapes from a rigid body. */
	typedef TArray<FPhysicsShapeHandle, TInlineAllocator<NumInlinedPxShapeElements>> FInlineShapeArray;
}


// LINEAR CCPT
UENUM()
enum EConstraintPlasticityType : int
{
	/** */
	CCPT_Free	UMETA(DisplayName = "Free"),
	/** */
	CCPT_Shrink	UMETA(DisplayName = "Shrink"),
	/** */
	CCPT_Grow	UMETA(DisplayName = "Grow"),

	CCPT_MAX,
};

// LINEAR DOF
UENUM()
enum ELinearConstraintMotion : int
{
	/** No constraint against this axis. */
	LCM_Free	UMETA(DisplayName="Free"),
	/** Limited freedom along this axis. */
	LCM_Limited UMETA(DisplayName="Limited"),
	/** Fully constraint against this axis. */
	LCM_Locked UMETA(DisplayName="Locked"),

	LCM_MAX,
};

/** This filter allows us to refine queries (channel, object) with an additional level of ignore by tagging entire classes of objects (e.g. "Red team", "Blue team")
    If(QueryIgnoreMask & ShapeFilter != 0) filter out */
typedef uint8 FMaskFilter;

namespace Chaos
{
enum class ESerializedDataContext : uint8;
struct FSerializedDataBufferHandle;
class FBVHParticles;

	template <typename T, int>
	class TPBDRigidParticles;

	class FPerParticleGravity;

	class FConvex;
	class FCapsule;

	template <typename T, int>
	class TAABB;
	using FAABB3 = TAABB<FReal, 3>;

	template <typename T, int>
	class TBox;

	template <typename T, int>
	class TSphere;
	using FSphere = TSphere<FReal, 3>;

	class FConstraintBase;
	class FJointConstraint;
	class FSuspensionConstraint;

	class FTriangleMeshImplicitObject;

	struct FSerializedDataBuffer;
	using FSerializedDataBufferPtr = TUniquePtr<FSerializedDataBuffer>;
}

struct FCollisionShape;
class UPhysicalMaterial;

class FPhysicsAggregateReference_Chaos
{
public:
	bool IsValid() const { return false; }
};

class FPhysicsConstraintReference_Chaos
{
public:
	FPhysicsConstraintReference_Chaos() { Reset(); }
	void Reset() { Constraint = nullptr; }

	PHYSICSCORE_API bool IsValid() const;



	Chaos::FConstraintBase* operator->() { return Constraint; }
	const Chaos::FConstraintBase* operator->() const { return Constraint; }

	Chaos::FConstraintBase* Constraint;
};

class FPhysicsShapeReference_Chaos
{
public:
	
	FPhysicsShapeReference_Chaos()
		: Shape(nullptr), ActorRef() { }
	FPhysicsShapeReference_Chaos(Chaos::FPerShapeData* ShapeIn, const FPhysicsActorHandle& ActorRefIn)
		: Shape(ShapeIn), ActorRef(ActorRefIn) { }

	bool IsValid() const { return (Shape != nullptr); }
	bool Equals(const FPhysicsShapeReference_Chaos& Other) const { return Shape == Other.Shape; }
    bool operator==(const FPhysicsShapeReference_Chaos& Other) const { return Equals(Other); }
	PHYSICSCORE_API const Chaos::FImplicitObject& GetGeometry() const;

	Chaos::FPerShapeData* Shape;
    FPhysicsActorHandle ActorRef;

	friend uint32 GetTypeHash(const FPhysicsShapeReference_Chaos& InShapeReference)
	{
		return PointerHash(InShapeReference.Shape);
	}
};

class FPhysicsShapeAdapter_Chaos
{
public:
	PHYSICSCORE_API FPhysicsShapeAdapter_Chaos(const FQuat& Rot, const FCollisionShape& CollisionShape);
	PHYSICSCORE_API ~FPhysicsShapeAdapter_Chaos();

	PHYSICSCORE_API const FPhysicsGeometry& GetGeometry() const;
	PHYSICSCORE_API FTransform GetGeomPose(const FVector& Pos) const;
	PHYSICSCORE_API const FQuat& GetGeomOrientation() const;

private:
	TRefCountPtr<FPhysicsGeometry> Geometry;
	FQuat GeometryRotation;
};

/**
 Wrapper around geometry. This is really just needed to make the physx chaos abstraction easier
 */
struct FPhysicsGeometryCollection_Chaos
{
	// Delete default constructor, want only construction by interface (private constructor below)
	FPhysicsGeometryCollection_Chaos() = delete;
	// No copying or assignment, move construction only, these are defaulted in the source file as they need
	// to be able to delete physx::PxGeometryHolder which is incomplete here
	FPhysicsGeometryCollection_Chaos(const FPhysicsGeometryCollection_Chaos& Copy) = delete;
	FPhysicsGeometryCollection_Chaos& operator=(const FPhysicsGeometryCollection_Chaos& Copy) = delete;
	PHYSICSCORE_API FPhysicsGeometryCollection_Chaos(FPhysicsGeometryCollection_Chaos&& Steal);
	FPhysicsGeometryCollection_Chaos& operator=(FPhysicsGeometryCollection_Chaos&& Steal) = delete;
	PHYSICSCORE_API ~FPhysicsGeometryCollection_Chaos();

	PHYSICSCORE_API ECollisionShapeType GetType() const;
	PHYSICSCORE_API const Chaos::FImplicitObject& GetGeometry() const;
	PHYSICSCORE_API const Chaos::TBox<Chaos::FReal, 3>& GetBoxGeometry() const;
	PHYSICSCORE_API const Chaos::FSphere&  GetSphereGeometry() const;
	PHYSICSCORE_API const Chaos::FCapsule&  GetCapsuleGeometry() const;
	PHYSICSCORE_API const Chaos::FConvex& GetConvexGeometry() const;
	PHYSICSCORE_API const Chaos::FTriangleMeshImplicitObject& GetTriMeshGeometry() const;

private:
	friend class FChaosEngineInterface;
	PHYSICSCORE_API explicit FPhysicsGeometryCollection_Chaos(const FPhysicsShapeReference_Chaos& InShape);
	PHYSICSCORE_API explicit FPhysicsGeometryCollection_Chaos(const FPhysicsGeometry& InGeom);

	const Chaos::FImplicitObject& Geom;
};


// Temp interface
namespace physx
{
    class PxActor;
    class PxScene;
	class PxSimulationEventCallback;
    class PxGeometry;
    class PxTransform;
    class PxQuat;
	class PxMassProperties;
}
struct FContactModifyCallback;
class ULineBatchComponent;

class FChaosEngineInterface
{
public:
	virtual ~FChaosEngineInterface() = default;

	static PHYSICSCORE_API void CreateActor(const FActorCreationParams& InParams,FPhysicsActorHandle& Handle, UObject* InOwner = nullptr);
	static PHYSICSCORE_API void ReleaseActor(FPhysicsActorHandle& InActorReference,FChaosScene* InScene = nullptr,bool bNeverDeferRelease=false);

	static bool IsValid(const FPhysicsActorHandle& Handle) { return Handle != nullptr; }
	static PHYSICSCORE_API void AddActorToSolver(const FPhysicsActorHandle& Handle,Chaos::FPhysicsSolver* Solver);
	static PHYSICSCORE_API void RemoveActorFromSolver(const FPhysicsActorHandle& Handle,Chaos::FPhysicsSolver* Solver);

	static PHYSICSCORE_API void SetGlobalPose_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FTransform& InNewPose,bool bAutoWake = true);
	static PHYSICSCORE_API void SetKinematicTarget_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FTransform& InNewTarget);

	static PHYSICSCORE_API FChaosScene* GetCurrentScene(const FPhysicsActorHandle& InHandle);

	static PHYSICSCORE_API FPhysicsMaterialHandle CreateMaterial(const UPhysicalMaterial* InMaterial);
	static PHYSICSCORE_API void UpdateMaterial(FPhysicsMaterialHandle& InHandle, UPhysicalMaterial* InMaterial);
	static PHYSICSCORE_API void ReleaseMaterial(FPhysicsMaterialHandle& InHandle);
	static PHYSICSCORE_API void SetUserData(FPhysicsMaterialHandle& InHandle,void* InUserData);

	static PHYSICSCORE_API void ReleaseMaterialMask(FPhysicsMaterialMaskHandle& InHandle);

	static PHYSICSCORE_API FPhysicsAggregateReference_Chaos CreateAggregate(int32 MaxBodies);
	static PHYSICSCORE_API void ReleaseAggregate(FPhysicsAggregateReference_Chaos& InAggregate);
	static PHYSICSCORE_API int32 GetNumActorsInAggregate(const FPhysicsAggregateReference_Chaos& InAggregate);
	static PHYSICSCORE_API void AddActorToAggregate_AssumesLocked(const FPhysicsAggregateReference_Chaos& InAggregate,const FPhysicsActorHandle& InActor);

	static PHYSICSCORE_API int32 GetNumShapes(const FPhysicsActorHandle& InHandle);

	static PHYSICSCORE_API void ReleaseShape(const FPhysicsShapeHandle& InShape);

	static PHYSICSCORE_API void AttachShape(const FPhysicsActorHandle& InActor,const FPhysicsShapeHandle& InNewShape);
	static PHYSICSCORE_API void DetachShape(const FPhysicsActorHandle& InActor,FPhysicsShapeHandle& InShape,bool bWakeTouching = true);

	static PHYSICSCORE_API void SetSmoothEdgeCollisionsEnabled_AssumesLocked(const FPhysicsActorHandle& InActor, const bool bSmoothEdgeCollisionsEnabled);

	static PHYSICSCORE_API void AddDisabledCollisionsFor_AssumesLocked(const TMap<FPhysicsActorHandle, TArray< FPhysicsActorHandle > >& InMap);
	static PHYSICSCORE_API void RemoveDisabledCollisionsFor_AssumesLocked(TArray< FPhysicsActorHandle > & InPhysicsActors);

	static PHYSICSCORE_API void SetDisabled(const FPhysicsActorHandle& InPhysicsActor, bool bSetDisabled);
	static PHYSICSCORE_API bool IsDisabled(const FPhysicsActorHandle& InPhysicsActor);

	static PHYSICSCORE_API void SetActorUserData_AssumesLocked(const FPhysicsActorHandle& InActorReference,FPhysicsUserData* InUserData);

	static PHYSICSCORE_API bool IsRigidBody(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API bool IsDynamic(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API bool IsStatic(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API bool IsKinematic(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API bool IsKinematic_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API bool IsSleeping(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API bool IsCcdEnabled(const FPhysicsActorHandle& InActorReference);
	// @todo(mlentine): We don't have a notion of sync vs async and are a bit of both. Does this work?
	static bool HasSyncSceneData(const FPhysicsActorHandle& InHandle) { return true; }
	static bool HasAsyncSceneData(const FPhysicsActorHandle& InHandle) { return false; }
	static PHYSICSCORE_API bool IsInScene(const FPhysicsActorHandle& InActorReference);

	static PHYSICSCORE_API int32 GetAllShapes_AssumedLocked(const FPhysicsActorHandle& InActorHandle,TArray<FPhysicsShapeReference_Chaos,FDefaultAllocator>& OutShapes);
	static PHYSICSCORE_API int32 GetAllShapes_AssumedLocked(const FPhysicsActorHandle& InActorHandle,PhysicsInterfaceTypes::FInlineShapeArray& OutShapes);

	static PHYSICSCORE_API bool CanSimulate_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API float GetMass_AssumesLocked(const FPhysicsActorHandle& InActorReference);

	static PHYSICSCORE_API void SetSendsSleepNotifies_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bSendSleepNotifies);
	static PHYSICSCORE_API void ClearWakeEvent_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void PutToSleep_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void WakeUp_AssumesLocked(const FPhysicsActorHandle& InActorReference);

	static PHYSICSCORE_API void SetIsKinematic_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bIsKinematic);
	static PHYSICSCORE_API void SetCcdEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bIsCcdEnabled);
	static PHYSICSCORE_API void SetMACDEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bIsMACDEnabled);
	static PHYSICSCORE_API void SetPartialIslandSleepAllowed_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bIsPartialIslandSleepAllowed);
	static PHYSICSCORE_API void SetPositionSolverIterationCount_AssumesLocked(const FPhysicsActorHandle& InActorReference, int32 PositionSolverIterationCount);
	static PHYSICSCORE_API void SetVelocitySolverIterationCount_AssumesLocked(const FPhysicsActorHandle& InActorReference, int32 VelocitySolverIterationCount);
	static PHYSICSCORE_API void SetProjectionSolverIterationCount_AssumesLocked(const FPhysicsActorHandle& InActorReference, int32 ProjectionSolverIterationCount);
	static PHYSICSCORE_API void SetIgnoreAnalyticCollisions_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bIsCcdEnabled);

	static PHYSICSCORE_API FTransform GetGlobalPose_AssumesLocked(const FPhysicsActorHandle& InActorReference);

	static PHYSICSCORE_API FTransform GetTransform_AssumesLocked(const FPhysicsActorHandle& InRef,bool bForceGlobalPose = false);

	static PHYSICSCORE_API bool HasKinematicTarget_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API FTransform GetKinematicTarget_AssumesLocked(const FPhysicsActorHandle& InActorReference);

	static PHYSICSCORE_API FVector GetLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InNewVelocity,bool bAutoWake = true);

	static PHYSICSCORE_API FVector GetAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InNewVelocity,bool bAutoWake = true);
	static PHYSICSCORE_API float GetMaxAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API float GetMaxLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetMaxAngularVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InMaxAngularVelocityRadians);
	static PHYSICSCORE_API void SetMaxLinearVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference, float InMaxLinearVelocity);

	static PHYSICSCORE_API float GetMaxDepenetrationVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetMaxDepenetrationVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InMaxDepenetrationVelocity);

	static PHYSICSCORE_API FVector GetWorldVelocityAtPoint_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InPoint);
	static PHYSICSCORE_API FVector GetWorldVelocityAtPoint_AssumesLocked(const Chaos::FRigidBodyHandle_Internal* InActorReference, const FVector& InPoint);

	static PHYSICSCORE_API FTransform GetComTransform_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API FTransform GetComTransformLocal_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetComTransformLocal_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FTransform& InTransform);

	static PHYSICSCORE_API FVector GetLocalInertiaTensor_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetLocalInertiaTensor_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& InInertiaTensor);
	static PHYSICSCORE_API FBox GetBounds_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API FBox GetBounds_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FTransform& InTransform);
	static PHYSICSCORE_API FBox GetBoundsLocal_AssumesLocked(const FPhysicsActorHandle& InActorReference);

	static PHYSICSCORE_API void SetLinearDamping_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InDamping);
	static PHYSICSCORE_API void SetAngularDamping_AssumesLocked(const FPhysicsActorHandle& InActorReference,float InDamping);

	static PHYSICSCORE_API void AddForce_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Force, bool bAllowSubstepping, bool bAccelChange, bool bIsInternal = false);
	static PHYSICSCORE_API void AddForceAtPosition_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Force, const FVector& Position, bool bAllowSubstepping, bool bIsLocalForce = false, bool bIsInternal = false);
	static PHYSICSCORE_API void AddRadialForce_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Origin, const float Radius, const float Strength, const uint8 Falloff, bool bAccelChange, bool bAllowSubstepping, bool bIsInternal = false);
	static PHYSICSCORE_API void AddTorque_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& Torque, bool bAllowSubstepping, bool bAccelChange, bool bIsInternal = false);

	static PHYSICSCORE_API void AddImpulse_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InForce, bool bIsInternal = false);
	static PHYSICSCORE_API void AddAngularImpulseInRadians_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InTorque, bool bIsInternal = false);
	static PHYSICSCORE_API void AddVelocity_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InForce, bool bIsInternal = false);
	static PHYSICSCORE_API void AddAngularVelocityInRadians_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InTorque, bool bIsInternal = false);
	static PHYSICSCORE_API void AddImpulseAtLocation_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InImpulse,const FVector& InLocation, bool bIsInternal = false);
	static PHYSICSCORE_API void AddVelocityChangeImpulseAtLocation_AssumesLocked(const FPhysicsActorHandle& InActorReference, const FVector& InVelocityDelta, const FVector& InLocation, bool bIsInternal = false);
	static PHYSICSCORE_API void AddRadialImpulse_AssumesLocked(const FPhysicsActorHandle& InActorReference,const FVector& InOrigin,float InRadius,float InStrength,ERadialImpulseFalloff InFalloff,bool bInVelChange, bool bIsInternal = false);

	static PHYSICSCORE_API bool IsGravityEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetGravityEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference,bool bEnabled);
	static PHYSICSCORE_API int32 GetGravityGroupIndex_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetGravityGroupIndex_AssumesLocked(const FPhysicsActorHandle& InActorReference, uint32 Index);

	static PHYSICSCORE_API bool GetUpdateKinematicFromSimulation_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetUpdateKinematicFromSimulation_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bUpdateKinematicFromSimulation);

	static PHYSICSCORE_API bool GetGyroscopicTorqueEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetGyroscopicTorqueEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bGyroscopicTorqueEnabled);

	static PHYSICSCORE_API void SetOneWayInteraction_AssumesLocked(const FPhysicsActorHandle& InHandle, bool InOneWayInteraction);

	static PHYSICSCORE_API float GetSleepEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetSleepEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InActorReference, float InEnergyThreshold);

	static PHYSICSCORE_API void SetSleepThresholdMultiplier_AssumesLocked(const FPhysicsActorHandle& InActorReference, float ThresholdMultiplier);

	static PHYSICSCORE_API void SetMass_AssumesLocked(const FPhysicsActorHandle& InHandle,float InMass);
	static PHYSICSCORE_API void SetMassSpaceInertiaTensor_AssumesLocked(const FPhysicsActorHandle& InHandle,const FVector& InTensor);
	static PHYSICSCORE_API void SetComLocalPose_AssumesLocked(const FPhysicsActorHandle& InHandle,const FTransform& InComLocalPose);

	static PHYSICSCORE_API bool IsInertiaConditioningEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference);
	static PHYSICSCORE_API void SetInertiaConditioningEnabled_AssumesLocked(const FPhysicsActorHandle& InActorReference, bool bEnabled);

	static PHYSICSCORE_API float GetStabilizationEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InHandle);
	static PHYSICSCORE_API void SetStabilizationEnergyThreshold_AssumesLocked(const FPhysicsActorHandle& InHandle,float InThreshold);
	static PHYSICSCORE_API void SetSolverPositionIterationCount_AssumesLocked(const FPhysicsActorHandle& InHandle, int32 InSolverIterationCount);
	static PHYSICSCORE_API void SetSolverVelocityIterationCount_AssumesLocked(const FPhysicsActorHandle& InHandle, int32 InSolverIterationCount);
	static PHYSICSCORE_API void SetSolverProjectionIterationCount_AssumesLocked(const FPhysicsActorHandle& InHandle, int32 InSolverIterationCount);
	static PHYSICSCORE_API float GetWakeCounter_AssumesLocked(const FPhysicsActorHandle& InHandle);
	static PHYSICSCORE_API void SetWakeCounter_AssumesLocked(const FPhysicsActorHandle& InHandle,float InWakeCounter);

	static PHYSICSCORE_API void SetInitialized_AssumesLocked(const FPhysicsActorHandle& InHandle,bool InInitialized);

	static PHYSICSCORE_API SIZE_T GetResourceSizeEx(const FPhysicsActorHandle& InActorRef);

	static PHYSICSCORE_API FPhysicsConstraintHandle CreateConstraint(Chaos::FPhysicsObject* Body1, Chaos::FPhysicsObject* Body2, const FTransform& InLocalFrame1, const FTransform& InLocalFrame2);
	static PHYSICSCORE_API FPhysicsConstraintHandle CreateConstraint(const FPhysicsActorHandle& InActorRef1,const FPhysicsActorHandle& InActorRef2,const FTransform& InLocalFrame1,const FTransform& InLocalFrame2);

	static PHYSICSCORE_API FPhysicsConstraintHandle CreateSuspension(Chaos::FPhysicsObject* Body, const FVector& InLocalFrame);
	static PHYSICSCORE_API FPhysicsConstraintHandle CreateSuspension(const FPhysicsActorHandle& InActorRef, const FVector& InLocalFrame);
	static PHYSICSCORE_API void SetConstraintUserData(const FPhysicsConstraintHandle& InConstraintRef,void* InUserData);
	static PHYSICSCORE_API void ReleaseConstraint(FPhysicsConstraintHandle& InConstraintRef);

	static PHYSICSCORE_API FTransform GetLocalPose(const FPhysicsConstraintHandle& InConstraintRef,EConstraintFrame::Type InFrame);
	static PHYSICSCORE_API FTransform GetGlobalPose(const FPhysicsConstraintHandle& InConstraintRef,EConstraintFrame::Type InFrame);
	static PHYSICSCORE_API FVector GetLocation(const FPhysicsConstraintHandle& InConstraintRef);
	static PHYSICSCORE_API void GetForce(const FPhysicsConstraintHandle& InConstraintRef,FVector& OutLinForce,FVector& OutAngForce);
	static PHYSICSCORE_API void GetDriveLinearVelocity(const FPhysicsConstraintHandle& InConstraintRef,FVector& OutLinVelocity);
	static PHYSICSCORE_API void GetDriveAngularVelocity(const FPhysicsConstraintHandle& InConstraintRef,FVector& OutAngVelocity);

	static PHYSICSCORE_API float GetCurrentSwing1(const FPhysicsConstraintHandle& InConstraintRef);
	static PHYSICSCORE_API float GetCurrentSwing2(const FPhysicsConstraintHandle& InConstraintRef);
	static PHYSICSCORE_API float GetCurrentTwist(const FPhysicsConstraintHandle& InConstraintRef);

	static PHYSICSCORE_API void SetCanVisualize(const FPhysicsConstraintHandle& InConstraintRef,bool bInCanVisualize);
	static PHYSICSCORE_API void SetCollisionEnabled(const FPhysicsConstraintHandle& InConstraintRef,bool bInCollisionEnabled);
	static PHYSICSCORE_API void SetProjectionEnabled_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef,bool bInProjectionEnabled,float InLinearAlpha = 1.0f,float InAngularAlpha = 0.0f, float InLinearTolerance = 0.0f, float InAngularToleranceDeg = 0.0f);
	static PHYSICSCORE_API void SetShockPropagationEnabled_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, bool bInShockPropagationEnabled, float InShockPropagationAlpha);
	static PHYSICSCORE_API void SetParentDominates_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, bool bInParentDominates);
	static PHYSICSCORE_API void SetMassConditioningEnabled_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, bool bInMassConditioningEnabled);
	static PHYSICSCORE_API void SetUseLinearJointSolver_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, bool bInUseLinearJointSolver);
	static PHYSICSCORE_API void SetBreakForces_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef,float InLinearBreakForce,float InAngularBreakForce);
	static PHYSICSCORE_API void SetViolationCallbackThresholds_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef,float InLinearBreakViolation,float InAngularBreakViolation);
	static PHYSICSCORE_API void SetPlasticityLimits_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InLinearPlasticityLimit, float InAngularPlasticityLimit, EConstraintPlasticityType InLinearPlasticityType);
	static PHYSICSCORE_API void SetContactTransferScale_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InContactTransferScale);
	static PHYSICSCORE_API void SetLocalPose(const FPhysicsConstraintHandle& InConstraintRef, const FTransform& InPose, EConstraintFrame::Type InFrame);

	static PHYSICSCORE_API void SetDrivePosition(const FPhysicsConstraintHandle& InConstraintRef,const FVector& InPosition);
	static PHYSICSCORE_API void SetDriveOrientation(const FPhysicsConstraintHandle& InConstraintRef,const FQuat& InOrientation);
	static PHYSICSCORE_API void SetDriveLinearVelocity(const FPhysicsConstraintHandle& InConstraintRef,const FVector& InLinVelocity);
	static PHYSICSCORE_API void SetDriveAngularVelocity(const FPhysicsConstraintHandle& InConstraintRef,const FVector& InAngVelocity);

	static PHYSICSCORE_API void SetTwistLimit(const FPhysicsConstraintHandle& InConstraintRef,float InLowerLimit,float InUpperLimit,float InContactDistance);
	static PHYSICSCORE_API void SetSwingLimit(const FPhysicsConstraintHandle& InConstraintRef,float InYLimit,float InZLimit,float InContactDistance);
	static PHYSICSCORE_API void SetLinearLimit(const FPhysicsConstraintHandle& InConstraintRef,float InLimit);

	static PHYSICSCORE_API bool IsBroken(const FPhysicsConstraintHandle& InConstraintRef);

	static PHYSICSCORE_API void SetGeometry(FPhysicsShapeHandle& InShape, Chaos::FImplicitObjectPtr&& InGeometry);

	UE_DEPRECATED(5.4, "Use SetGeometry with FImplicitObjectPtr instead.")
	static void SetGeometry(FPhysicsShapeHandle& InShape, TUniquePtr<Chaos::FImplicitObject>&& InGeometry) {check(false);}
	
	UE_DEPRECATED(5.7, "Use GetCombinedShapeFilterData instead") static PHYSICSCORE_API FCollisionFilterData GetSimulationFilter(const FPhysicsShapeHandle& InShape);
	UE_DEPRECATED(5.7, "Use GetCombinedShapeFilterData instead") static PHYSICSCORE_API FCollisionFilterData GetQueryFilter(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API Chaos::Filter::FShapeFilterData GetShapeFilterData(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API Chaos::Filter::FInstanceData GetFilterInstanceData(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API Chaos::Filter::FCombinedShapeFilterData GetCombinedShapeFilterData(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API bool IsSimulationShape(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API bool IsQueryShape(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API ECollisionShapeType GetShapeType(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API FTransform GetLocalTransform(const FPhysicsShapeHandle& InShape);


	static PHYSICSCORE_API void* GetUserData(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API void SetUserData(const FPhysicsShapeHandle& InShape, void* InUserData);
	static PHYSICSCORE_API FPhysicsShapeHandle CloneShape(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API FPhysicsGeometryCollection_Chaos GetGeometryCollection(const FPhysicsShapeHandle& InShape);
	static PHYSICSCORE_API FPhysicsGeometryCollection_Chaos GetGeometryCollection(const FPhysicsGeometry& InShape);

	// @todo(mlentine): Which of these do we need to support?
	// Set the mask filter of a shape, which is an extra level of filtering during collision detection / query for extra channels like "Blue Team" and "Red Team"
	static PHYSICSCORE_API void SetMaskFilter(const FPhysicsShapeHandle& InShape,FMaskFilter InFilter);
	UE_DEPRECATED(5.7, "Use SetCombinedShapeFilterData instead") static PHYSICSCORE_API void SetSimulationFilter(const FPhysicsShapeHandle& InShape, const FCollisionFilterData& InFilter);
	UE_DEPRECATED(5.7, "Use SetCombinedShapeFilterData instead") static PHYSICSCORE_API void SetQueryFilter(const FPhysicsShapeHandle& InShape, const FCollisionFilterData& InFilter);
	static PHYSICSCORE_API void SetShapeFilterData(const FPhysicsShapeHandle& InShape, const Chaos::Filter::FShapeFilterData& InFilter);
	static PHYSICSCORE_API void SetFilterInstanceData(const FPhysicsShapeHandle& InShape, const Chaos::Filter::FInstanceData& InFilter);
	static PHYSICSCORE_API void SetCombinedShapeFilterData(const FPhysicsShapeHandle& InShape, const Chaos::Filter::FCombinedShapeFilterData& InFilter);
	static PHYSICSCORE_API void SetIsSimulationShape(const FPhysicsShapeHandle& InShape,bool bIsSimShape);
	static PHYSICSCORE_API void SetIsProbeShape(const FPhysicsShapeHandle& InShape,bool bIsProbeShape);
	static PHYSICSCORE_API void SetIsQueryShape(const FPhysicsShapeHandle& InShape,bool bIsQueryShape);
	static PHYSICSCORE_API void SetLocalTransform(const FPhysicsShapeHandle& InShape,const FTransform& NewLocalTransform);

	UE_INTERNAL static PHYSICSCORE_API void SerializeParticleStateToBuffer(const FPhysicsActorHandle& InActorHandle, Chaos::FSerializedDataBuffer& OutSerializedData, Chaos::ESerializedDataContext DataContext);
	UE_INTERNAL static PHYSICSCORE_API void ApplySerializedStateToParticle(const FPhysicsActorHandle& InActorHandle, TUniquePtr<Chaos::FSerializedDataBuffer>&& InSerializedData, Chaos::ESerializedDataContext DataContext);

	UE_INTERNAL static PHYSICSCORE_API void SerializeConstraintStateToBuffer(const FPhysicsConstraintHandle& ConstraintHandle, Chaos::FSerializedDataBuffer& OutSerializedData, Chaos::ESerializedDataContext DataContext);
	UE_INTERNAL static PHYSICSCORE_API void ApplySerializedStateToConstraint(const FPhysicsConstraintHandle& ConstraintHandle, Chaos::FSerializedDataBufferPtr&& InSerializedData, Chaos::ESerializedDataContext DataContext);
};
