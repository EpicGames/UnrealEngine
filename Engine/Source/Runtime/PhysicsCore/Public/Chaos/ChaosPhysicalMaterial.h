// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ChaosPhysicalMaterial.generated.h"

namespace Chaos
{
	class FChaosPhysicsMaterial;
}

/**
 * Physical materials are used to define the response of a physical object when 
 * interacting dynamically with the world.
 */
UCLASS(BlueprintType, Blueprintable, CollapseCategories, HideCategories = Object, MinimalAPI)
class UChaosPhysicalMaterial : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Update the values in \p Mat to reflect the values in this \c UObject. */
	PHYSICSCORE_API void CopyTo(Chaos::FChaosPhysicsMaterial& Mat) const;

	//
	// Surface properties.
	//
	
	/** Friction value of a surface in motion, controls how easily things can slide on this surface (0 is frictionless, higher values increase the amount of friction) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta=(ClampMin=0))
	float Friction;

	/** Friction value of surface at rest, controls how easily things can slide on this surface (0 is frictionless, higher values increase the amount of friction) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (ClampMin = 0))
	float StaticFriction;

	/** Restitution or 'bounciness' of this surface, between 0 (no bounce) and 1 (outgoing velocity is same as incoming). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta=(ClampMin=0))
	float Restitution;

	/** Uniform linear ether drag, the resistance a body experiences to its translation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PhysicalMaterial", meta=(ClampMin = 0))
	float LinearEtherDrag;

	/** Uniform angular ether drag, the resistance a body experiences to its rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PhysicalMaterial", meta = (ClampMin = 0))
	float AngularEtherDrag;

	/** How much to scale the damage threshold by on any destructible we are applied to */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta=(ClampMin=0))
	float SleepingLinearVelocityThreshold;
	
	/** How much to scale the damage threshold by on any destructible we are applied to */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta=(ClampMin=0))
	float SleepingAngularVelocityThreshold;
};



