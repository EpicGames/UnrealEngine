// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/GCObject.h"
#include "Chaos/ChaosEngineInterface.h"

//Chaos includes. Todo: move to chaos core so we can include for all of engine
#include "Chaos/Declares.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "Framework/Threading.h"
#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Async/TaskGraphInterfaces.h"
#include "Chaos/SimCallbackInput.h"
#include "Chaos/SimCallbackObject.h"

#include "ChaosVisualDebugger/ChaosVDContextProvider.h"

#include "ChaosDebugDraw/ChaosDDTypes.h"

#ifndef CHAOS_DEBUG_NAME
#define CHAOS_DEBUG_NAME 0
#endif

// Currently compilation issue with Incredibuild when including headers required by event template functions
#define XGE_FIXED 0

class AdvanceOneTimeStepTask;
class FChaosSolversModule;
struct FForceFieldProxy;
struct FSolverStateStorage;

class FSkeletalMeshPhysicsProxy;
class FStaticMeshPhysicsProxy;
class FPerSolverFieldSystem;

class IPhysicsProxyBase;

namespace Chaos
{
	class FPhysicsProxy;

	struct FCollisionEventData;

	enum class EEventType : int32;

	template<typename PayloadType, typename HandlerType>
	class TRawEventHandler;

	class FAccelerationStructureHandle;

	template <typename TPayload, typename T, int d>
	class ISpatialAcceleration;

	template <typename TPayload, typename T, int d>
	class ISpatialAccelerationCollection;

	template <typename T>
	class TArrayCollectionArray;

	class FPBDRigidDirtyParticlesBufferAccessor;
}

struct FChaosSceneCallbackInput : public Chaos::FSimCallbackInput
{
	Chaos::FVec3 Gravity;

	void Reset() {}
};

struct FChaosSceneSimCallback : public Chaos::TSimCallbackObject<FChaosSceneCallbackInput>
{
	virtual void OnPreSimulate_Internal() override;
	virtual FName GetFNameForStatId() const override;
};

/**
* Low level Chaos scene used when building custom simulations that don't exist in the main world physics scene.
*/
class FChaosScene
#if WITH_ENGINE
	: public FGCObject
#endif
{
public:
	PHYSICSCORE_API FChaosScene(
		UObject* OwnerPtr
		, Chaos::FReal InAsyncDt
		, const FName& DebugName = NAME_None
	);

	PHYSICSCORE_API virtual ~FChaosScene();

	/**
	 * Get the internal Chaos solver object
	 */
	Chaos::FPhysicsSolver* GetSolver() const { return SceneSolver; }

#if WITH_ENGINE
	// FGCObject Interface ///////////////////////////////////////////////////
	PHYSICSCORE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const
	{
		return "FChaosScene";
	}
	//////////////////////////////////////////////////////////////////////////
#endif
	
	PHYSICSCORE_API const Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* GetSpacialAcceleration() const;
	PHYSICSCORE_API Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* GetSpacialAcceleration();

	PHYSICSCORE_API void AddActorsToScene_AssumesLocked(TArray<FPhysicsActorHandle>& InHandles,const bool bImmediate=true);
	PHYSICSCORE_API void RemoveActorFromAccelerationStructure(FPhysicsActorHandle Actor);
	PHYSICSCORE_API void RemoveActorFromAccelerationStructureImp(Chaos::FGeometryParticle* Particle);
	PHYSICSCORE_API void UpdateActorsInAccelerationStructure(const TArrayView<FPhysicsActorHandle>& Actors);
	PHYSICSCORE_API void UpdateActorInAccelerationStructure(const FPhysicsActorHandle& Actor);

	PHYSICSCORE_API void WaitPhysScenes();

	/**
	 * Copies the acceleration structure out of the solver, does no thread safety checking so ensure calls
	 * to this are made at appropriate sync points if required
	 */
	PHYSICSCORE_API void CopySolverAccelerationStructure();

	/**
	 * Flushes all pending global, task and solver command queues and refreshes the spatial acceleration
	 * for the scene. Required when querying against a currently non-running scene to ensure the scene
	 * is correctly represented
	 */
	PHYSICSCORE_API void Flush();
#if WITH_EDITOR
	PHYSICSCORE_API void AddPieModifiedObject(UObject* InObj);
#endif

	PHYSICSCORE_API void StartFrame();
	PHYSICSCORE_API void SetUpForFrame(const FVector* NewGrav,float InDeltaSeconds,float InMinPhysicsDeltaTime,float InMaxPhysicsDeltaTime,float InMaxSubstepDeltaTime,int32 InMaxSubsteps,bool bSubstepping);
	PHYSICSCORE_API void EndFrame();

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPhysScenePostTick,FChaosScene*);
	FOnPhysScenePostTick OnPhysScenePostTick;

	PHYSICSCORE_API void KillSafeAsyncTasks();
	PHYSICSCORE_API void WaitSolverTasks();
	PHYSICSCORE_API bool AreAnyTasksPending() const;
	PHYSICSCORE_API void BeginDestroy();
	PHYSICSCORE_API bool IsCompletionEventComplete() const;
	PHYSICSCORE_API FGraphEventArray GetCompletionEvents();

	void SetNetworkDeltaTimeScale(float InDeltaTimeScale) { MNetworkDeltaTimeScale = InDeltaTimeScale; }
	float GetNetworkDeltaTimeScale() const { return MNetworkDeltaTimeScale; }

protected:

	Chaos::ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* SolverAccelerationStructure;

	// Control module for Chaos - cached to avoid constantly hitting the module manager
	FChaosSolversModule* ChaosModule;

	// Solver representing this scene
	Chaos::FPhysicsSolver* SceneSolver;

#if WITH_EDITOR
	// List of objects that we modified during a PIE run for physics simulation caching.
	TArray<TObjectPtr<UObject>> PieModifiedObjects;
#endif

	// Allow other code to obtain read-locks when needed
	friend struct ChaosInterface::FScopedSceneReadLock;
	friend struct FScopedSceneLock_Chaos;

	//Engine interface BEGIN
	virtual float OnStartFrame(float InDeltaTime){ return InDeltaTime; }
	PHYSICSCORE_API virtual void OnSyncBodies(Chaos::FPhysicsSolverBase* Solver);
	//Engine interface END

	float MDeltaTime;
	float MNetworkDeltaTimeScale = 1.f; // Scale passed in delta time by this. Used by NetworkPrediction to make clients slow down or catch up when needed

	UObject* Owner;

private:
	/** Get all the physics solvers with the same owner */
	PHYSICSCORE_API TArray<Chaos::FPhysicsSolverBase*> GetPhysicsSolvers() const;

	PHYSICSCORE_API void SetGravity(const Chaos::FVec3& Acceleration);

	template <typename TSolver>
	void SyncBodies(TSolver* Solver);

	// Taskgraph control
	FGraphEventArray CompletionEvents;

	FChaosSceneSimCallback* SimCallback;

#if WITH_CHAOS_VISUAL_DEBUGGER
private:
	FChaosVDContext CVDContextData;

public:
	FChaosVDContext& GetChaosVDContextData()
	{
		return CVDContextData;
	};
#endif

#if CHAOS_DEBUG_DRAW
public:
	const ChaosDD::Private::FChaosDDScenePtr& GetDebugDrawScene() const
	{ 
		return CDDScene;
	}

	PHYSICSCORE_API void SetDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& CDDScene);

private:
	ChaosDD::Private::FChaosDDScenePtr CDDScene;
#endif
};
