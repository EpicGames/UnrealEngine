// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PhysicsInterfaceTypesCore.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FChaosUserDefinedEntity
{
public:
	FChaosUserDefinedEntity() = delete;
	FChaosUserDefinedEntity(FName InEntityTypeName) : EntityTypeName(InEntityTypeName) {};
	virtual ~FChaosUserDefinedEntity() = default;

	FName GetEntityTypeName() {
		return EntityTypeName;
	}

	// Get the UObject (usually a verse::component) that owns this FChaosUserDefinedEntity
	virtual TWeakObjectPtr<UObject> GetOwnerObject() = 0;

private:	
	FName EntityTypeName;
};

// This is used to add your own Unreal Engine agnostic user entities to Chaos Physics Results
struct FChaosUserEntityAppend : public FChaosUserData
{
	FChaosUserEntityAppend()
		: FChaosUserData()
		, ChaosUserData(nullptr)
		, UserDefinedEntity(nullptr)
	{ 
		Type = EChaosUserDataType::ChaosUserEntity;
		Payload = this;
	}

	// Get the UObject that controls the lifetime of the physics objects
	// that will be referencing this UserData object. It will be stored
	// as a weak pointer in hit/overlap results to verify that the raw
	// PhysicsObject pointer they hold is valid.
	TWeakObjectPtr<UObject> GetOwnerObject()
	{
		if (UserDefinedEntity != nullptr)
		{
			return UserDefinedEntity->GetOwnerObject();
		}
		return TWeakObjectPtr<UObject>();
	}

	FChaosUserData* ChaosUserData; //  Chaos data to access the physics body properties
	FChaosUserDefinedEntity* UserDefinedEntity;
};
