// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/CollisionFilterData.h"
#include "Chaos/Declares.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/Real.h"
#include "ChaosSQTypes.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "PhysXPublicCore.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceTypesCore.h"
#include "PhysicsInterfaceWrapperShared.h"

class UPhysicalMaterial;

namespace Chaos
{
	class FCapsule;
	class FImplicitObject;
}

namespace ChaosInterface
{
struct FDummyPhysType {};
struct FDummyPhysActor {};

template<typename DummyT>
struct FDummyCallback {};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
using FQueryFilterData UE_DEPRECATED(5.8, "Use ChaosInterface::FSceneQueryCommonParams instead.") = FChaosQueryFilterData;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** We use this struct so that if no conversion is needed in another API, we can avoid the copy (if we think that's critical) */
struct FPhysicsRaycastInputAdapater
{
	FPhysicsRaycastInputAdapater(const FVector& InStart, const FVector& InDir, const EHitFlags InFlags)
		: Start(InStart)
		, Dir(InDir)
		, OutputFlags(InFlags)
	{

	}
	FVector Start;
	FVector Dir;
	EHitFlags OutputFlags;
};

/** We use this struct so that if no conversion is needed in another API, we can avoid the copy (if we think that's critical) */
struct FPhysicsSweepInputAdapater
{
	FPhysicsSweepInputAdapater(const FTransform& InStartTM, const FVector& InDir, const EHitFlags InFlags)
		: StartTM(InStartTM)
		, Dir(InDir)
		, OutputFlags(InFlags)
	{

	}
	FTransform StartTM;
	FVector Dir;
	EHitFlags OutputFlags;
};

/** We use this struct so that if no conversion is needed in another API, we can avoid the copy (if we think that's critical) */
struct FPhysicsOverlapInputAdapater
{
	FPhysicsOverlapInputAdapater(const FTransform& InPose)
		: GeomPose(InPose)
	{

	}
	FTransform GeomPose;
};

/** This is used to add debug data to scene query visitors in non-shipping builds */
struct FQueryDebugParams
{
#if CHAOS_DEBUG_DRAW
	FQueryDebugParams()
		: bDebugQuery(false)
		, bExternalQuery(true) { }
	bool bDebugQuery;
	bool bExternalQuery;
	bool IsDebugQuery() const { return bDebugQuery; }
	bool IsExternalQuery() const { return bExternalQuery; }
#else
	// In test or shipping builds, this struct must be left empty
	FQueryDebugParams() { }
	constexpr bool IsDebugQuery() const { return false; }
	constexpr bool IsExternalQuery() const { return true; }
#endif
};

UE_DEPRECATED(5.7, "Use GetCombinedShapeFilterData instead")  extern PHYSICSCORE_API FCollisionFilterData GetQueryFilterData(const Chaos::FPerShapeData& Shape);
UE_DEPRECATED(5.7, "Use GetCombinedShapeFilterData instead")  extern PHYSICSCORE_API FCollisionFilterData GetSimulationFilterData(const Chaos::FPerShapeData& Shape);
extern PHYSICSCORE_API Chaos::Filter::FShapeFilterData GetShapeFilterData(const Chaos::FPerShapeData& Shape);
extern PHYSICSCORE_API Chaos::Filter::FInstanceData GetFilterInstanceData(const Chaos::FPerShapeData& Shape);
extern PHYSICSCORE_API Chaos::Filter::FCombinedShapeFilterData GetCombinedShapeFilterData(const Chaos::FPerShapeData& Shape);


PHYSICSCORE_API ECollisionShapeType GetImplicitType(const Chaos::FImplicitObject& InGeometry);

inline ECollisionShapeType GetType(const Chaos::FImplicitObject& InGeometry)
{
	return GetImplicitType(InGeometry);
}

PHYSICSCORE_API Chaos::FReal GetRadius(const Chaos::FCapsule& InCapsule);

PHYSICSCORE_API Chaos::FReal GetHalfHeight(const Chaos::FCapsule& InCapsule);


inline bool HadInitialOverlap(const FLocationHit& Hit)
{
	return Hit.Distance <= 0.f;
}

inline bool HadInitialOverlap(const FPTLocationHit& Hit)
{
	return Hit.Distance <= 0.f;
}

inline const Chaos::FPerShapeData* GetShape(const FActorShape& Hit)
{
	return Hit.Shape;
}

inline const Chaos::FPerShapeData* GetShape(const FPTActorShape& Hit)
{
	return Hit.Shape;
}

inline Chaos::FGeometryParticle* GetActor(const FActorShape& Hit)
{
	return Hit.Actor;
}

inline Chaos::FGeometryParticleHandle* GetActor(const FPTActorShape& Hit)
{
	return Hit.Actor;
}

inline Chaos::FReal GetDistance(const FLocationHit& Hit)
{
	return Hit.Distance;
}

inline Chaos::FReal GetDistance(const FPTLocationHit& Hit)
{
	return Hit.Distance;
}

inline FVector GetPosition(const FLocationHit& Hit)
{
	return Hit.WorldPosition;
}

inline FVector GetPosition(const FPTLocationHit& Hit)
{
	return Hit.WorldPosition;
}

inline FVector GetNormal(const FLocationHit& Hit)
{
	return Hit.WorldNormal;
}

inline FVector GetNormal(const FPTLocationHit& Hit)
{
	return Hit.WorldNormal;
}

inline FHitFlags GetFlags(const FLocationHit& Hit)
{
	return Hit.Flags;
}

inline FHitFlags GetFlags(const FPTLocationHit& Hit)
{
	return Hit.Flags;
}


FORCEINLINE void SetFlags(FLocationHit& Hit, FHitFlags Flags)
{
	Hit.Flags = Flags;
}

FORCEINLINE void SetFlags(FPTLocationHit& Hit, FHitFlags Flags)
{
	Hit.Flags = Flags;
}

inline uint32 GetInternalFaceIndex(const FQueryHit& Hit)
{
	return Hit.FaceIndex;
}

inline uint32 GetInternalFaceIndex(const FPTQueryHit& Hit)
{
	return Hit.FaceIndex;
}

inline void SetInternalFaceIndex(FQueryHit& Hit, uint32 FaceIndex)
{
	Hit.FaceIndex = FaceIndex;
}

inline void SetInternalFaceIndex(FPTQueryHit& Hit, uint32 FaceIndex)
{
	Hit.FaceIndex = FaceIndex;
}


inline uint32 GetInvalidPhysicsFaceIndex()
{
	return 0xffffffff;
}

inline uint32 GetTriangleMeshExternalFaceIndex(const FDummyPhysType& Shape, uint32 InternalFaceIndex)
{
	return GetInvalidPhysicsFaceIndex();
}

inline FTransform GetGlobalPose(const FDummyPhysActor& RigidActor)
{
	return FTransform::Identity;
}

inline uint32 GetNumShapes(const FDummyPhysActor& RigidActor)
{
	return 0;
}

inline void GetShapes(const FDummyPhysActor& RigidActor, Chaos::FImplicitObject** ShapesBuffer, uint32 NumShapes)
{

}

inline void SetActor(FDummyPhysType& Hit, FDummyPhysActor* Actor)
{

}

inline void SetShape(FDummyPhysType& Hit, Chaos::FImplicitObject* Shape)
{

}

template <typename HitType>
HitType* GetBlock(FSQHitBuffer<HitType>& Callback)
{
	return Callback.GetBlock();
}

template <typename HitType>
bool GetHasBlock(const FSQHitBuffer<HitType>& Callback)
{
	return Callback.HasBlockingHit();
}

} // namespace ChaosInterface
