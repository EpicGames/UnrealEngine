// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_CHAOS_VISUAL_DEBUGGER

#include "CollisionQueryFilterCallbackCore.h"
#include "Chaos/ISpatialAcceleration.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"

/** Initializes a CVD Wrapper for a SQ visit with the provider data, and traces it as soon it goes iut of scope if we are recording */
#ifndef CVD_TRACE_SCOPED_SCENE_QUERY_VISIT_HELPER
	#define CVD_TRACE_SCOPED_SCENE_QUERY_VISIT_HELPER(Type, ParticleTransform, ParticleIndex, ShapeIndex, CurData) \
	CVD_SCOPED_DATA_CHANNEL_OVERRIDE(CVDDC_SceneQueries) \
	FChaosVDQueryVisitStep CVDSQVisitStepData; \
	CVD_TRACE_SCOPED_SCENE_QUERY_VISIT(CVDSQVisitStepData); \
	Chaos::VisualDebugger::TraceHelpers::FillCVDQueryVisitData(CVDSQVisitStepData, Type, ParticleTransform, ParticleIndex, ShapeIndex, CurData);
#endif

/** Fill a hit data entry on the current CVD Wrapper for a SQ visit in scope if we are recording
 * This needs to be called in the same scope of a CVD_TRACE_SCOPED_SCENE_QUERY_VISIT_HELPER call
 */
#ifndef CVD_FILL_HIT_DATA_HELPER
	#define CVD_FILL_HIT_DATA_HELPER(Hit, HitType) \
	Chaos::VisualDebugger::TraceHelpers::FillCVDHitDataHelper(Hit, HitType, CVDSQVisitStepData);
#endif

/** Sets a reject reason to the current scene query being recorded - Compiled out if we don't have CVD support */
#ifndef CVD_SET_SQ_SHAPE_REJECT_REASON
	#define CVD_SET_SQ_SHAPE_REJECT_REASON(Reason) \
			CVDSQVisitStepData.RejectReason = Reason;
#endif

namespace Chaos::VisualDebugger::TraceHelpers
{
	template <class THitType>
	void FillCVDHitDataHelper(THitType& Hit, ECollisionQueryHitType HitType, FChaosVDQueryVisitStep& InOutVisitData);
	inline void FillCVDQueryVisitData(FChaosVDQueryVisitStep& InOutVisitData, EChaosVDSceneQueryVisitorType Type, const FTransform& ParticleTransform, int32 ParticleIndex, int32 ShapeIndex, const Chaos::FQueryFastData* CurData)
	{
		if (!FChaosVisualDebuggerTrace::IsTracing())
		{
			return;
		}

		InOutVisitData.Type = Type;
		InOutVisitData.ParticleTransform = ParticleTransform;
		InOutVisitData.ParticleIndex = ParticleIndex;
		InOutVisitData.ShapeIndex = ShapeIndex;

		if (CurData)
		{
			InOutVisitData.QueryFastData.Dir = CurData->Dir;
			InOutVisitData.QueryFastData.InvDir = CurData->InvDir;
			InOutVisitData.QueryFastData.CurrentLength = CurData->CurrentLength;
			InOutVisitData.QueryFastData.InvCurrentLength = CurData->InvCurrentLength;
			InOutVisitData.QueryFastData.bParallel0 = CurData->bParallel[0];
			InOutVisitData.QueryFastData.bParallel1 = CurData->bParallel[1];
			InOutVisitData.QueryFastData.bParallel2 = CurData->bParallel[2];
			InOutVisitData.QueryFastData.MarkAsValid();
		}

		InOutVisitData.MarkAsValid();
	}

	template <class THitType>
	void FillCVDHitDataHelper(THitType& Hit, ECollisionQueryHitType HitType, FChaosVDQueryVisitStep& InOutVisitData)
	{
		if (!FChaosVisualDebuggerTrace::IsTracing())
		{
			return;
		}

		if constexpr (std::is_base_of_v<ChaosInterface::FLocationHit, THitType> || std::is_base_of_v<ChaosInterface::FPTLocationHit, THitType>)
		{
			InOutVisitData.HitData.Distance = Hit.Distance;
			InOutVisitData.HitData.WorldPosition = Hit.WorldPosition;
			InOutVisitData.HitData.WorldNormal = Hit.WorldNormal;
			InOutVisitData.HitData.Flags = static_cast<uint16>(Hit.Flags.HitFlags);
			InOutVisitData.HitData.FaceIdx = Hit.FaceIndex;
		}

		if constexpr (std::is_base_of_v<ChaosInterface::FQueryHit, THitType> || std::is_base_of_v<ChaosInterface::FPTQueryHit, THitType>)
		{
			InOutVisitData.HitData.FaceNormal = Hit.FaceNormal;
		}

		InOutVisitData.HitType = static_cast<EChaosVDCollisionQueryHitType>(HitType);

		InOutVisitData.HitData.MarkAsValid();
	}
}

#else
	#ifndef CVD_TRACE_SCOPED_SCENE_QUERY_VISIT_HELPER
		#define CVD_TRACE_SCOPED_SCENE_QUERY_VISIT_HELPER(Type, ParticleTransform, ParticleIndex, ShapeIndex, CurData)
	#endif
	#ifndef CVD_FILL_HIT_DATA_HELPER
		#define CVD_FILL_HIT_DATA_HELPER(Hit, HitType)
	#endif
	#ifndef CVD_SET_SQ_SHAPE_REJECT_REASON
		#define CVD_SET_SQ_SHAPE_REJECT_REASON(RejectReason)
	#endif
#endif
