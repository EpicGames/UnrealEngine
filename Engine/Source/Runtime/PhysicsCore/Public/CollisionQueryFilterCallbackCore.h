// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsInterfaceTypesCore.h"
#include "ChaosInterfaceWrapperCore.h"

/**
 *
 * Make sure this matches PxQueryHitType for HitTypeToPxQueryHitType to work
 */
enum class ECollisionQueryHitType : uint8
{
	None = 0,
	Touch = 1,
	Block = 2
};

namespace Chaos
{
	class FImplicitObject;
}

class ICollisionQueryFilterCallback
{
public:
	using FQueryFilterData = Chaos::Filter::FQueryFilterData;

	virtual ~ICollisionQueryFilterCallback() = default;
	
	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) = 0;
	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) = 0;

	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) = 0;
	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) = 0;
};

class FBlockAllCollisionQueryFilterCallback : public ICollisionQueryFilterCallback
{
public:
	virtual ~FBlockAllCollisionQueryFilterCallback() = default;

	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) { return ECollisionQueryHitType::Block; }
	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) { return ECollisionQueryHitType::Block; }

	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) { return ECollisionQueryHitType::Block; }
	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) { return ECollisionQueryHitType::Block; }
};

class FOverlapAllCollisionQueryFilterCallback : public ICollisionQueryFilterCallback
{
public:
	virtual ~FOverlapAllCollisionQueryFilterCallback() = default;

	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) { return ECollisionQueryHitType::Touch; }
	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) { return ECollisionQueryHitType::Touch; }

	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) { return ECollisionQueryHitType::Touch; }
	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) { return ECollisionQueryHitType::Touch; }
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class UE_DEPRECATED(5.8, "Use ICollisionQueryFilterCallback instead") ICollisionQueryFilterCallbackBase : public ICollisionQueryFilterCallback
{
public:
	
	virtual ~ICollisionQueryFilterCallbackBase() = default;
	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) = 0;
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) = 0;

	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) = 0;
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) = 0;

private:
	static FCollisionFilterData GetLegacyFilterData(const FQueryFilterData& FilterData)
	{
		return Chaos::Filter::FQueryFilterBuilder::GetLegacyQueryFilter(FilterData);
	}

	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) override
	{
		return PostFilter(GetLegacyFilterData(FilterData), Hit);
	}

	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) override
	{
		return PreFilter(GetLegacyFilterData(FilterData), Shape, Actor);
	}

	virtual ECollisionQueryHitType PostFilter(const FQueryFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) override
	{
		return PostFilter(GetLegacyFilterData(FilterData), Hit);
	}

	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) override
	{
		return PreFilter(GetLegacyFilterData(FilterData), Shape, Actor);
	}
};

class UE_DEPRECATED(5.8, "Use FBlockAllCollisionQueryFilterCallback instead") FBlockAllQueryCallback : public ICollisionQueryFilterCallbackBase
{
public:
	virtual ~FBlockAllQueryCallback() = default;
	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) override { return ECollisionQueryHitType::Block; }
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) override { return ECollisionQueryHitType::Block; }

	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) override { return ECollisionQueryHitType::Block; }
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) override { return ECollisionQueryHitType::Block; }
};

class UE_DEPRECATED(5.8, "Use FOverlapAllCollisionQueryFilterCallback instead") FOverlapAllQueryCallback : public ICollisionQueryFilterCallbackBase
{
public:
	virtual ~FOverlapAllQueryCallback() = default;
	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FQueryHit& Hit) override { return ECollisionQueryHitType::Touch; }
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) override { return ECollisionQueryHitType::Touch; }

	virtual ECollisionQueryHitType PostFilter(const FCollisionFilterData& FilterData, const ChaosInterface::FPTQueryHit& Hit) override { return ECollisionQueryHitType::Touch; }
	virtual ECollisionQueryHitType PreFilter(const FCollisionFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) override { return ECollisionQueryHitType::Touch; }
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
