// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Math/Vector.h"

/** Types of Collision Shapes that are used by Trace **/
namespace ECollisionShape
{
	enum Type
	{
		Line,
		Box,
		Sphere,
		Capsule
	};
};

/** Collision Shapes that supports Sphere, Capsule, Box, or Line **/
struct FCollisionShape
{
	ECollisionShape::Type ShapeType;

	static constexpr float MinBoxExtent() { return UE_KINDA_SMALL_NUMBER; }
	static constexpr float MinSphereRadius() { return UE_KINDA_SMALL_NUMBER; }
	static constexpr float MinCapsuleRadius() { return UE_KINDA_SMALL_NUMBER; }
	static constexpr float MinCapsuleAxisHalfHeight() { return UE_KINDA_SMALL_NUMBER; }

	/** Union that supports up to 3 floats **/
	union
	{
		struct
		{
			float HalfExtentX;
			float HalfExtentY;
			float HalfExtentZ;
		} Box;

		struct
		{
			float Radius;
		} Sphere;

		struct
		{
			float Radius;
			float HalfHeight;
		} Capsule;
	};

	FCollisionShape()
	{
		ShapeType = ECollisionShape::Line;
	}

	/** Is the shape currently a Line (Default)? */
	bool IsLine() const
	{
		return ShapeType == ECollisionShape::Line;
	}

	/** Is the shape currently a box? */
	bool IsBox() const
	{
		return ShapeType == ECollisionShape::Box;
	}

	/** Is the shape currently a sphere? */
	bool IsSphere() const
	{
		return ShapeType == ECollisionShape::Sphere;
	}

	/** Is the shape currently a capsule? */
	bool IsCapsule() const
	{
		return ShapeType == ECollisionShape::Capsule;
	}

	/** Utility function to Set Box and dimension */
	void SetBox(const FVector3f& HalfExtent)
	{
		ShapeType = ECollisionShape::Box;
		Box.HalfExtentX = HalfExtent.X;
		Box.HalfExtentY = HalfExtent.Y;
		Box.HalfExtentZ = HalfExtent.Z;
	}

	/** Utility function to set Sphere with Radius */
	void SetSphere(const float Radius)
	{
		ShapeType = ECollisionShape::Sphere;
		Sphere.Radius = Radius;
	}

	/** Utility function to set Capsule with Radius and Half Height. Note: This is the full half-height (needs to include the sphere radius) */
	void SetCapsule(const float Radius, const float HalfHeight)
	{
		ShapeType = ECollisionShape::Capsule;
		Capsule.Radius = Radius;
		Capsule.HalfHeight = HalfHeight;
	}

	/** Utility function to set Capsule from Extent data */
	void SetCapsule(const FVector3f& Extent)
	{
		ShapeType = ECollisionShape::Capsule;
		Capsule.Radius = FMath::Max(Extent.X, Extent.Y);
		Capsule.HalfHeight = Extent.Z;
	}

	/** Utility function to set a shape from a type and extent */
	void SetShape(const ECollisionShape::Type InShapeType, const FVector& Extent)
	{
		switch (InShapeType)
		{
		case ECollisionShape::Box:
			{
				SetBox(FVector3f(Extent));
			}
			break;
		case ECollisionShape::Sphere:
			{
				SetSphere(static_cast<float>(Extent[0]));
			}
			break;
		case ECollisionShape::Capsule:
			{
				SetCapsule(FVector3f(Extent));
			}
			break;
		case ECollisionShape::Line:
		default:
			ShapeType = InShapeType;
		}
	}

	/** Return true if nearly zero. If so, it will back out and use line trace instead */
	bool IsNearlyZero() const
	{
		switch (ShapeType)
		{
		case ECollisionShape::Box:
		{
			return (Box.HalfExtentX <= FCollisionShape::MinBoxExtent() && Box.HalfExtentY <= FCollisionShape::MinBoxExtent() && Box.HalfExtentZ <= FCollisionShape::MinBoxExtent());
		}
		case ECollisionShape::Sphere:
		{
			return (Sphere.Radius <= FCollisionShape::MinSphereRadius());
		}
		case ECollisionShape::Capsule:
		{
			// @Todo check height? It didn't check before, so I'm keeping this way for time being
			return (Capsule.Radius <= FCollisionShape::MinCapsuleRadius());
		}
		}

		return true;
	}

	/** Utility function to return Extent of the shape */
	FVector GetExtent() const
	{
		switch (ShapeType)
		{
		case ECollisionShape::Box:
		{
			return FVector(Box.HalfExtentX, Box.HalfExtentY, Box.HalfExtentZ);
		}
		case ECollisionShape::Sphere:
		{
			return FVector(Sphere.Radius, Sphere.Radius, Sphere.Radius);
		}
		case ECollisionShape::Capsule:
		{
			// @Todo check height? It didn't check before, so I'm keeping this way for time being
			return FVector(Capsule.Radius, Capsule.Radius, Capsule.HalfHeight);
		}
		}

		return FVector::ZeroVector;
	}

	/** Get distance from center of capsule to center of sphere ends */
	float GetCapsuleAxisHalfLength() const
	{
		ensure(ShapeType == ECollisionShape::Capsule);
		return FMath::Max<float>(Capsule.HalfHeight - Capsule.Radius, FCollisionShape::FCollisionShape::MinCapsuleAxisHalfHeight());
	}

	/** Utility function to get Box Extention */
	FVector GetBox() const
	{
		return FVector(Box.HalfExtentX, Box.HalfExtentY, Box.HalfExtentZ);
	}

	/** Utility function to get Sphere Radius */
	const float GetSphereRadius() const
	{
		return Sphere.Radius;
	}

	/** Utility function to get Capsule Radius */
	const float GetCapsuleRadius() const
	{
		return Capsule.Radius;
	}

	/** Utility function to get Capsule Half Height. Note: This is the full half height which includes the sphere radius */
	const float GetCapsuleHalfHeight() const
	{
		return Capsule.HalfHeight;
	}

	FCollisionShape Inflate(const float Inflation) const
	{
		FCollisionShape InflatedShape;

		if (Inflation == 0.f)
		{
			InflatedShape = *this;
		}
		else
		{
			// Don't shrink below zero size.
			switch (ShapeType)
			{
			case ECollisionShape::Box:
				{
					const FVector3f InflatedExtent = FVector3f(Box.HalfExtentX + Inflation, Box.HalfExtentY + Inflation, Box.HalfExtentZ + Inflation).ComponentMax(FVector3f::ZeroVector);
					InflatedShape.SetBox(InflatedExtent);
				}
				break;
			case ECollisionShape::Sphere:
				{
					InflatedShape.SetSphere(FMath::Max(Sphere.Radius + Inflation, 0));
				}
				break;
			case ECollisionShape::Capsule:
				{
					InflatedShape.SetCapsule(FMath::Max(Capsule.Radius + Inflation, 0), FMath::Max(Capsule.HalfHeight + Inflation, 0));
				}
				break;
			case ECollisionShape::Line:
			default:
				{
					// do not inflate for unsupported shapes
					InflatedShape = *this;
				}
			}
		}

		return InflatedShape;
	}

	inline FString ToString() const
	{
		switch (ShapeType)
		{
		case ECollisionShape::Box:
			{
				return FString::Printf(TEXT("Box=(X=%3.3f Y=%3.3f Z=%3.3f)"), Box.HalfExtentX, Box.HalfExtentY, Box.HalfExtentZ);
			}
		case ECollisionShape::Sphere:
			{
				return FString::Printf(TEXT("Sphere=(Radius=%3.3f)"), Sphere.Radius);
			}
		case ECollisionShape::Capsule:
			{
				return FString::Printf(TEXT("Capsule=(Radius=%3.3f HalfHeight=%3.3f)"), Capsule.Radius, Capsule.HalfHeight);
			}
		case ECollisionShape::Line:
		default:
			{
				return TEXT("Line");
			}
		}
	}

	/** Used by engine in multiple places. Since LineShape doesn't need any dimension, declare once and used by all codes. */
	static struct FCollisionShape LineShape;

	/** Static utility function to make a box */
	static FCollisionShape MakeBox(const FVector& BoxHalfExtent)
	{
		FCollisionShape BoxShape;
		BoxShape.SetBox(FVector3f(BoxHalfExtent));
		return BoxShape;
	}

	/** Static utility function to make a box */
	static FCollisionShape MakeBox(const FVector3f& BoxHalfExtent)
	{
		FCollisionShape BoxShape;
		BoxShape.SetBox(BoxHalfExtent);
		return BoxShape;
	}

	/** Static utility function to make a sphere */
	static FCollisionShape MakeSphere(const float SphereRadius)
	{
		FCollisionShape SphereShape;
		SphereShape.SetSphere(SphereRadius);
		return SphereShape;
	}

	/** Static utility function to make a capsule. Note: This is the full half-height (needs to include the sphere radius) */
	static FCollisionShape MakeCapsule(const float CapsuleRadius, const float CapsuleHalfHeight)
	{
		FCollisionShape CapsuleShape;
		CapsuleShape.SetCapsule(CapsuleRadius, CapsuleHalfHeight);
		return CapsuleShape;
	}

	/** Static utility function to make a capsule */
	static FCollisionShape MakeCapsule(const FVector& Extent)
	{
		FCollisionShape CapsuleShape;
		CapsuleShape.SetCapsule(FVector3f(Extent));
		return CapsuleShape;
	}
};
