// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

// Vertex indices necessary to describe the vertices listed in TriMeshCollisionData
struct FTriIndices
{
	int32 v0;
	int32 v1;
	int32 v2;

	FTriIndices()
		: v0(0)
		, v1(0)
		, v2(0)
	{
	}
};

#if WITH_EDITORONLY_DATA
PHYSICSCORE_API void operator<<(class FArchive& Ar, FTriIndices& TriIndices);
#endif

// Description of triangle mesh collision data necessary for cooking physics data
struct FTriMeshCollisionData
{
	/** Array of vertices included in the triangle mesh */
	TArray<FVector3f> Vertices;

	/** Array of indices defining the ordering of triangles in the mesh */
	TArray<FTriIndices> Indices;

	/** Array of optional material indices (must equal num triangles) */
	TArray<uint16>	MaterialIndices;

	/** Optional UV co-ordinates (each array must be zero of equal num vertices) */
	TArray< TArray<FVector2D> > UVs;

	/** Does the mesh require its normals flipped (see PxMeshFlag) */
	uint32 bFlipNormals : 1;

	/** If mesh is deformable, we don't clean it, so that vertex layout does not change and it can be updated */
	uint32 bDeformableMesh : 1;

	/** Prioritize cooking speed over runtime speed */
	uint32 bFastCook : 1;

	/** Turn off ActiveEdgePrecompute (This makes cooking faster, but will slow contact generation) */
	uint32 bDisableActiveEdgePrecompute : 1;

	/** Use a reduced memory representation for the triangle mesh data, which may have additional query cost */
	uint32 bReducedMemoryRepresentation : 1;

	FTriMeshCollisionData()
		: bFlipNormals(false)
		, bDeformableMesh(false)
		, bFastCook(false)
		, bDisableActiveEdgePrecompute(false)
		, bReducedMemoryRepresentation(false)
	{
	}
};

#if WITH_EDITORONLY_DATA
PHYSICSCORE_API void operator<<(class FArchive& Ar, FTriMeshCollisionData& TriMeshCollisionData);
#endif

// Estimates of triangle mesh collision data necessary for cooking physics data
struct FTriMeshCollisionDataEstimates
{
	int64 VerticeCount = 0;
};