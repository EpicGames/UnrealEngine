// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define PHYSX_DDC TEXT("A08609EF508D460CBB6710540B03AACF");