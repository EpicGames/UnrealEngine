// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosArchive.h"
#include "Chaos/Defines.h"
#include "Chaos/HandleArray.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/ParticleHandle.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "SQCapture.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/MemoryReader.h"
#include "Templates/UniquePtr.h"

#ifndef PHYS_TEST_SERIALIZER
#define PHYS_TEST_SERIALIZER 1
#endif

// Utility used for serializing just physics data. This is meant for only the physics engine data (physx or chaos, not any unreal side). It is not meant to be used for actual serialization

#if PHYS_TEST_SERIALIZER

namespace Chaos
{

	class FChaosArchive;
}

class FPhysTestSerializer
{
public:
	PHYSICSCORE_API FPhysTestSerializer();
	FPhysTestSerializer(const FPhysTestSerializer& Other) = delete;
	FPhysTestSerializer(FPhysTestSerializer&& Other) = delete;
	FPhysTestSerializer& operator=(const FPhysTestSerializer&) = delete;
	FPhysTestSerializer& operator=(FPhysTestSerializer&&) = delete;

	UE_DEPRECATED(5.8, "Use the Serializer that takes an FArchive")
	PHYSICSCORE_API void Serialize(Chaos::FChaosArchive& Ar);
	PHYSICSCORE_API void Serialize(FArchive& Ar);
	PHYSICSCORE_API void Serialize(const TCHAR* FilePrefix);

	//Set the data from an external source. This will obliterate any existing data. Make sure you are not holding on to old internal data as it will go away
	PHYSICSCORE_API void SetPhysicsData(Chaos::FPBDRigidsEvolution& ChaosEvolution);

	const Chaos::FChaosArchiveContext* GetChaosContext() const
	{
		return &ChaosContext.Get();
	}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "FSQCapture is unused and will be removed")
	FSQCapture& CaptureSQ()
	{
		ensure(!SQCapture);	//we don't have support for multi sweeps yet since the scene could change
		SQCapture = TUniquePtr<FSQCapture>(new FSQCapture(*this));
		return *SQCapture;
	}

	UE_DEPRECATED(5.8, "FSQCapture is unused and will be removed")
	const FSQCapture* GetSQCapture()
	{
		if (SQCapture)
		{
			//todo: this sucks, find a better way to create data instead of doing it lazily
			GetChaosData();
#if 0 
			SQCapture->CreateChaosDataFromPhysX();
#endif
		}
		return SQCapture.Get();
	}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Chaos::FPBDRigidsEvolution* GetChaosData()
	{
#if 0
		if (!bChaosDataReady)
		{
			ensure(!bDiskDataIsChaos);
			//only supported for physx to chaos - don't have serialization context
			CreateChaosData();
		}
#endif
		return ChaosEvolution.Get();
	}

private:

#if 0
	PHYSICSCORE_API void CreateChaosData();
#endif 

private:

	class FArchiveWithExternalContext : public Chaos::FChaosArchive
	{
	public:
		FArchiveWithExternalContext(FArchive& ArIn)
			: FChaosArchive(ArIn)
		{
		}

		explicit FArchiveWithExternalContext(FArchive& ArIn, const TSharedRef<Chaos::FChaosArchiveContext>& InExternalSharedContext)
			: FChaosArchive(ArIn, InExternalSharedContext)
		{
		}
	};

	TArray<uint8> Data;	//how the data is stored before going to disk
	bool bDiskDataIsChaos;
	bool bChaosDataReady;

	
PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.8, "FSQCapture is unused and will be removed")
	TUniquePtr<FSQCapture> SQCapture;

PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TUniquePtr<Chaos::FPBDRigidsEvolution> ChaosEvolution;
	Chaos::FParticleUniqueIndicesMultithreaded UniqueIndices;
	Chaos::FPBDRigidsSOAs Particles;
	Chaos::THandleArray<Chaos::FChaosPhysicsMaterial> PhysicalMaterials;
	TArray <TUniquePtr<Chaos::FGeometryParticle>> GTParticles;

	TSharedRef<Chaos::FChaosArchiveContext> ChaosContext;

	FCustomVersionContainer ArchiveVersion;
};

#endif
