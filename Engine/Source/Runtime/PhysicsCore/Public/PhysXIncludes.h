// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.4, "PhysXIncludes.h is no longer useful.")

#include "CoreMinimal.h"