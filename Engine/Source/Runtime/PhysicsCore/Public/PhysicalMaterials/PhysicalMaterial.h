// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosEngineInterface.h"
#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceTypesCore.h"
#include "PhysicsSettingsEnums.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "PhysicalMaterial.generated.h"

struct FPropertyChangedEvent;

namespace PhysicalMaterialCVars
{
	extern bool bShowExperimentalProperties;
}


/**
 * Defines the directional strengths of a physical material in term of force per surface area
 */
USTRUCT(BlueprintType)
struct FPhysicalMaterialStrength
{
	GENERATED_USTRUCT_BODY()

	FPhysicalMaterialStrength();

	/**
	* Tensile strength of the material in MegaPascal ( 10^6 N/m2 )
	* This amount of tension force per area the material can withstand before it fractures
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PhysicalMaterial|Strength", meta = (ClampMin = 0, ForceUnits = "MPa"))
	float TensileStrength;

	/**
	* Compression strength of the material in MegaPascal ( 10^6 N/m2 )
	* This amount of compression force per area the material can withstand before it fractures, crumbles or buckles
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PhysicalMaterial|Strength", meta = (ClampMin = 0, ForceUnits = "MPa"))
	float CompressionStrength;

	/**
	* Shear strength of the material in MegaPascal ( 10^6 N/m2 )
	* This amount of shear force per area the material can withstand before it fractures
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PhysicalMaterial|Strength", meta = (ClampMin = 0, ForceUnits = "MPa"))
	float ShearStrength;
};


/**
 * Damage threshold modifiers, used by the Chaos destruction system
 */
USTRUCT(BlueprintType)
struct FPhysicalMaterialDamageModifier
{
	GENERATED_USTRUCT_BODY()

	FPhysicalMaterialDamageModifier();

	/**
	* Multiplier for the geometry collection damage thresholds/ internal strain
	* this allows for setting up unit damage threshold and use the material to scale them to the desired range of values
	* Note that the geometry collection asset needs to opt-in for the material modifer to be able to use it 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PhysicalMaterial|DamageModifier", meta = (ClampMin = 0))
	float DamageThresholdMultiplier;
};

/**
* Soft collision mode for a physical material.
* 
* NOTE: Must match EChaosPhysicsMaterialSoftCollisionMode
*/
UENUM()
enum class EPhysicalMaterialSoftCollisionMode : uint8
{
	// No soft collisionss
	None,

	// SoftCollisionThickess is a fraction of the bounds (minimum axis). Should be less than 0.5
	RelativeThickness,

	// SoftCollisionThickess is an absolute value in cm
	AbsoluteThickess,
};


/**
 * Physical materials are used to define the response of a physical object when interacting dynamically with the world.
 */
UCLASS(BlueprintType, Blueprintable, CollapseCategories, HideCategories = Object, MinimalAPI)
class UPhysicalMaterial : public UObject
{
	GENERATED_UCLASS_BODY()

	PHYSICSCORE_API virtual ~UPhysicalMaterial();
	PHYSICSCORE_API UPhysicalMaterial(FVTableHelper& Helper);
	//
	// Surface properties.
	//
	
	/** Friction value of surface, controls how easily things can slide on this surface (0 is frictionless, higher values increase the amount of friction) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=PhysicalMaterial, meta=(ClampMin=0))
	float Friction;

	/** Static Friction value of surface, controls how easily things can slide on this surface (0 is frictionless, higher values increase the amount of friction) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (ClampMin = 0))
	float StaticFriction;

	/** Friction combine mode, controls how friction is computed for multiple materials. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (editcondition = "bOverrideFrictionCombineMode"))
	TEnumAsByte<EFrictionCombineMode::Type> FrictionCombineMode;

	/** If set we will use the FrictionCombineMode of this material, instead of the FrictionCombineMode found in the project settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalMaterial)
	bool bOverrideFrictionCombineMode;

	/** Restitution or 'bounciness' of this surface, between 0 (no bounce) and 1 (outgoing velocity is same as incoming). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial,  meta=(ClampMin=0, ClampMax=1))
	float Restitution;

	/** Restitution combine mode, controls how restitution is computed for multiple materials. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (editcondition = "bOverrideRestitutionCombineMode"))
	TEnumAsByte<EFrictionCombineMode::Type> RestitutionCombineMode;

	/** If set we will use the RestitutionCombineMode of this material, instead of the RestitutionCombineMode found in the project settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalMaterial)
	bool bOverrideRestitutionCombineMode;

	//
	// Object properties.
	//
	
	/** Used with the shape of the object to calculate its mass properties. The higher the number, the heavier the object. g per cubic cm. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (ClampMin = 0, ForceUnits = "g/cm3"))
	float Density;

	/**  How low the linear velocity can be before solver puts body to sleep. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (ClampMin = 0))
	float SleepLinearVelocityThreshold;

	/** How low the angular velocity can be before solver puts body to sleep. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (ClampMin = 0))
	float SleepAngularVelocityThreshold;

	/** How many ticks we can be under thresholds for before solver puts body to sleep. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalMaterial, meta = (ClampMin = 0))
	int32 SleepCounterThreshold;

	/** 
	 *	Used to adjust the way that mass increases as objects get larger. This is applied to the mass as calculated based on a 'solid' object. 
	 *	In actuality, larger objects do not tend to be solid, and become more like 'shells' (e.g. a car is not a solid piece of metal).
	 *	Values are clamped to 1 or less.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Advanced, meta=(ClampMin=0.1, ClampMax=1))
	float RaiseMassToPower;

	/** How much to scale the damage threshold by on any destructible we are applied to */
	UE_DEPRECATED(5.3, "This property is not used anywhere, use Geometry Collection damage threshold related features instead")
	UPROPERTY()
	float DestructibleDamageThresholdScale_DEPRECATED;

	UPROPERTY()
	TObjectPtr<class UDEPRECATED_PhysicalMaterialPropertyBase> PhysicalMaterialProperty_DEPRECATED;

	/**
	 * To edit surface type for your project, use ProjectSettings/Physics/PhysicalSurface section
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalProperties)
	TEnumAsByte<EPhysicalSurface> SurfaceType;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalProperties)
	FPhysicalMaterialStrength Strength;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PhysicalProperties)
	FPhysicalMaterialDamageModifier DamageModifier;

	/** Color used to represent this material in certain debug views (e.g. landscape collision) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Debugging", AdvancedDisplay)
	FLinearColor DebugColor;

	//
	// Experimental properties
	//

	/** Experimental material properties are enabled via the p.PhysicalMaterial.ShowExperimentalProperties console variable.
		NOTE: These are _experimental_ properties which may change. Use at your own risk! */
	UPROPERTY(VisibleAnywhere, Transient, Category = "Experimental")
	bool bShowExperimentalProperties = PhysicalMaterialCVars::bShowExperimentalProperties;

	/** For enable soft collision shell thickness mode */
	UPROPERTY(EditAnywhere, Category = "Experimental|Softness", meta = (EditCondition = "bShowExperimentalProperties"))
	EPhysicalMaterialSoftCollisionMode SoftCollisionMode;

	/** Thickness of the layer just inside the collision shape in which contact is considered "soft".
		The units depend on SoftCollisionMode */
	UPROPERTY(EditAnywhere, Category = "Experimental|Softness", meta = (ClampMin = 0, EditCondition = "bShowExperimentalProperties"))
	float SoftCollisionThickness;

	/** A friction (positional) impulse of at least this magnitude may be applied,
		regardless the normal force. This is analogous to adding only the lateral part of a
		"stickiness" to a material */
	UPROPERTY(EditAnywhere, Category = "Experimental|Stickiness", meta = (ClampMin = 0, Units = kgcm, EditCondition = "bShowExperimentalProperties"))
	float BaseFrictionImpulse;

public:

	TUniquePtr<FPhysicsMaterialHandle> MaterialHandle;

	FChaosUserData UserData;

	//~ Begin UObject Interface
#if WITH_EDITOR
	PHYSICSCORE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	static PHYSICSCORE_API void RebuildPhysicalMaterials();
#endif // WITH_EDITOR
	PHYSICSCORE_API virtual void PostLoad() override;
	PHYSICSCORE_API virtual void FinishDestroy() override;
	//~ End UObject Interface

	PHYSICSCORE_API virtual void Serialize(FArchive& Ar) override;

	/** Get the physics-interface derived version of this material */
	PHYSICSCORE_API FPhysicsMaterialHandle& GetPhysicsMaterial();

	static PHYSICSCORE_API void SetEngineDefaultPhysMaterial(UPhysicalMaterial* Material);

	static PHYSICSCORE_API void SetEngineDefaultDestructiblePhysMaterial(UPhysicalMaterial* Material);

	/** Determine Material Type from input PhysicalMaterial **/
	static PHYSICSCORE_API EPhysicalSurface DetermineSurfaceType(UPhysicalMaterial const* PhysicalMaterial);

private:

	void EnsurePhysicsMaterialCreated();
};



