// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This is the PhysicalMaterialPropertyBase which the PhysicalMaterial has.
 * Individual games should derive their own MyGamPhysicalMaterialProperty.
 *
 * Then inside that object you can either have a bunch of properties or have it 
 * point to your game specific objects.
 *
 * (e.g.  You have have impact sounds and impact effects for all of the weapons
 * in your game.  So you have an object which contains the data needed per
 * material type and then you have your MyGamePhysicalMaterialProperty point to 
 * that. )
 *
 * class UMyGamePhysicalMaterialProperty extends UPhysicalMaterialPropertyBase
 *    editinlinenew;
 *
 * editable() editinline MyGameSpecificImpactEffects ImpactEffects;
 * editable() editinline MyGameSpecificImpactSounds ImpactSounds;
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Chaos/ChaosEngineInterface.h"
#include "PhysicalMaterialPropertyBase.generated.h"

UCLASS(collapsecategories, hidecategories=Object, editinlinenew, abstract,MinimalAPI, deprecated)
class UDEPRECATED_PhysicalMaterialPropertyBase : public UObject
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	virtual bool IsAsset() const override { return false; }
	//~ End UObject Interface

	// compatibility support function
	virtual EPhysicalSurface ConvertToSurfaceType(){ return SurfaceType_Default;  }

protected:
	// set material type from int
	EPhysicalSurface GetSurfaceType(int32 Value) 
	{ 
		// if you hit this, we might have to increase MaterialType count
		check (Value < SurfaceType_Max); 
		return (EPhysicalSurface)Value; 
	}
};

