// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Physics Core module interface
 */
class IPhysicsCoreModule : public IModuleInterface
{
public:
};
