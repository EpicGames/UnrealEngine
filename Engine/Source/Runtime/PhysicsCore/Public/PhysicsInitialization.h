// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "PhysicsCore.h"

namespace PhysDLLHelper
{
	/**
	 *	Load the required modules for PhysX
	 */
	PHYSICSCORE_API bool LoadPhysXModules(bool bLoadCooking);

	/**
	 *	Unload the required modules for PhysX
	 */
	PHYSICSCORE_API void UnloadPhysXModules();
}

bool PHYSICSCORE_API InitGamePhysCore();
void PHYSICSCORE_API TermGamePhysCore();