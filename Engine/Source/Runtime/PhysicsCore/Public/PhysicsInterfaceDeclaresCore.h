// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define MAX_PHYSICS_FIELD_TARGETS 32

#include "ChaosSQTypes.h"
#include "Chaos/ShapeInstanceFwd.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"

namespace Chaos
{
	class FImplicitObject;

	class FCapsule;

	template <typename T, int d>
	class TGeometryParticle;
	using FGeometryParticle = TGeometryParticle<FReal, 3>;

	struct FMassProperties;

	class FPerShapeData;

	class FPhysicalMaterial;
	class FPhysicalMaterialMask;

	template<typename, uint32, uint32>
	class THandle;

	struct FMaterialHandle;
	struct FMaterialMaskHandle;

	class FChaosPhysicsMaterial;
	class FChaosPhysicsMaterialMask;
}

// Temporary dummy types until SQ implemented
namespace ChaosInterface
{
	struct FDummyPhysType;
	struct FDummyPhysActor;
	template<typename T> struct FDummyCallback;
	struct FScopedSceneReadLock;
}
using FPhysTypeDummy = ChaosInterface::FDummyPhysType;
using FPhysActorDummy = ChaosInterface::FDummyPhysActor;

template<typename T>
using FCallbackDummy = ChaosInterface::FDummyCallback<T>;

using FHitLocation = ChaosInterface::FLocationHit;
using FHitSweep = ChaosInterface::FSweepHit;
using FHitRaycast = ChaosInterface::FRaycastHit;
using FHitOverlap = ChaosInterface::FOverlapHit;
using FPhysicsQueryHit = ChaosInterface::FQueryHit;

using FPhysicsTransform = FTransform;

using FPhysicsShape = Chaos::FPerShapeData;
using FPhysicsGeometry = Chaos::FImplicitObject;
using FPhysicsCapsuleGeometry = Chaos::FCapsule;
using FPhysicsMaterial = Chaos::FChaosPhysicsMaterial;
using FPhysicsMaterialMask = Chaos::FChaosPhysicsMaterialMask; 
using FPhysicsActor = Chaos::FGeometryParticle;

template <typename T>
using FPhysicsHitCallback = ChaosInterface::FSQHitBuffer<T>;

template <typename T>
using FSingleHitBuffer = ChaosInterface::FSQSingleHitBuffer<T>;

template <typename T>
using FDynamicHitBuffer = ChaosInterface::FSQHitBuffer<T>;

using FPhysicsActorHandle = Chaos::FSingleParticlePhysicsProxy*;

class FChaosSceneId;
class FPhysInterface_Chaos;
class FPhysicsConstraintReference_Chaos;
class FPhysicsAggregateReference_Chaos;
class FPhysicsShapeReference_Chaos;
class FPhysScene_Chaos;
class FPhysicsShapeAdapter_Chaos;
struct FPhysicsGeometryCollection_Chaos;
class FPhysicsUserData_Chaos;

typedef FPhysicsConstraintReference_Chaos	FPhysicsConstraintHandle;
typedef FPhysInterface_Chaos				FPhysicsInterface;
typedef FPhysScene_Chaos					FPhysScene;
typedef FPhysicsAggregateReference_Chaos	FPhysicsAggregateHandle;
typedef FPhysInterface_Chaos				FPhysicsCommand;
typedef FPhysicsShapeReference_Chaos		FPhysicsShapeHandle;
typedef FPhysicsGeometryCollection_Chaos	FPhysicsGeometryCollection;
typedef Chaos::FMaterialHandle				FPhysicsMaterialHandle;
typedef Chaos::FMaterialMaskHandle			FPhysicsMaterialMaskHandle;
typedef FPhysicsShapeAdapter_Chaos			FPhysicsShapeAdapter;
typedef FPhysicsUserData_Chaos				FPhysicsUserData;

inline FPhysicsActorHandle DefaultPhysicsActorHandle() { return nullptr; }
