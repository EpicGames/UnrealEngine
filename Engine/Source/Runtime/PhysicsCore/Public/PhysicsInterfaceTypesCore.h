// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceWrapperShared.h"
#include "Chaos/CollisionFilterData.h"

struct FBodyInstanceCore;
struct FChaosQueryFlag;
class FChaosScene;

struct FActorCreationParams
{
	FActorCreationParams()
		: Scene(nullptr)
		, InitialTM(FTransform::Identity)
		, bStatic(false)
		, bQueryOnly(false)
		, bEnableGravity(false)
		, bUpdateKinematicFromSimulation(false)
		, bSimulatePhysics(false)
		, bStartAwake(true)
		, DebugName(nullptr)
	{}
	FChaosScene* Scene;
	FTransform InitialTM;
	bool bStatic;
	bool bQueryOnly;
	bool bEnableGravity;
	bool bUpdateKinematicFromSimulation;
	bool bSimulatePhysics;
	bool bStartAwake;
	char* DebugName;
};

/**
* Type of query for object type or trace type
* Trace queries correspond to trace functions with TravelChannel/ResponseParams
* Object queries correspond to trace functions with Object types
*/
enum class ECollisionQuery : uint8
{
	ObjectQuery = 0,
	TraceQuery = 1
};

enum class ECollisionShapeType : uint8
{
	Sphere,
	Plane,
	Box,
	Capsule,
	Convex,
	Trimesh,
	Heightfield,
	None
};

/** Helper struct holding physics body filter data during initialisation */
struct FBodyCollisionFilterData
{
	// These must be explicitly declared to get rid of deprecation warnings.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FBodyCollisionFilterData() = default;
	FBodyCollisionFilterData(const FBodyCollisionFilterData&) = default;
	FBodyCollisionFilterData(FBodyCollisionFilterData&&) = default;

	FBodyCollisionFilterData& operator=(const FBodyCollisionFilterData&) = default;
	FBodyCollisionFilterData& operator=(FBodyCollisionFilterData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.8, "Use the FilterInstanceData and Simple/Complex ShapeFilterData objects instead")
	FCollisionFilterData SimFilter;
	UE_DEPRECATED(5.8, "Use the FilterInstanceData and Simple/Complex ShapeFilterData objects instead")
	FCollisionFilterData QuerySimpleFilter;
	UE_DEPRECATED(5.8, "Use the FilterInstanceData and Simple/Complex ShapeFilterData objects instead")
	FCollisionFilterData QueryComplexFilter;

	Chaos::Filter::FInstanceData FilterInstanceData;
	Chaos::Filter::FShapeFilterData SimpleShapeFilterData;
	Chaos::Filter::FShapeFilterData ComplexShapeFilterData;
};

struct FBodyCollisionFlags
{
	FBodyCollisionFlags()
		: bEnableSimCollisionSimple(false)
		, bEnableSimCollisionComplex(false)
		, bEnableQueryCollision(false)
		, bEnableProbeCollision(false)
	{
	}

	bool bEnableSimCollisionSimple;
	bool bEnableSimCollisionComplex;
	bool bEnableQueryCollision;
	bool bEnableProbeCollision;
};


/** Helper object to hold initialisation data for shapes */
struct FBodyCollisionData
{
	FBodyCollisionFilterData CollisionFilterData;
	FBodyCollisionFlags CollisionFlags;
};

inline void SetupNonUniformHelper(FVector InScale3D, double& OutMinScale, double& OutMinScaleAbs, FVector& OutScale3DAbs)
{
	// if almost zero, set min scale
	// @todo fixme
	if (InScale3D.IsNearlyZero())
	{
		// set min scale
		InScale3D = FVector(0.1f);
	}

	OutScale3DAbs = InScale3D.GetAbs();
	OutMinScaleAbs = OutScale3DAbs.GetMin();

	OutMinScale = FMath::Max3(InScale3D.X, InScale3D.Y, InScale3D.Z) < 0.f ? -OutMinScaleAbs : OutMinScaleAbs;	//if all three values are negative make minScale negative

	if (FMath::IsNearlyZero(OutMinScale))
	{
		// only one of them can be 0, we make sure they have mini set up correctly
		OutMinScale = 0.1f;
		OutMinScaleAbs = 0.1f;
	}
}

inline void SetupNonUniformHelper(FVector InScale3D, float& OutMinScale, float& OutMinScaleAbs, FVector& OutScale3DAbs)
{
	double OutMinScaleD, OutMinScaleAbsD;
	SetupNonUniformHelper(InScale3D, OutMinScaleD, OutMinScaleAbsD, OutScale3DAbs);
	OutMinScale = static_cast<float>(OutMinScaleD);	// LWC_TODO: Precision loss?
	OutMinScaleAbs = static_cast<float>(OutMinScaleAbsD);
}


/** Util to determine whether to use NegX version of mesh, and what transform (rotation) to apply. */
inline bool CalcMeshNegScaleCompensation(const FVector& InScale3D, FTransform& OutTransform)
{
	OutTransform = FTransform::Identity;

	if (InScale3D.Y > 0.f)
	{
		if (InScale3D.Z > 0.f)
		{
			// no rotation needed
		}
		else
		{
			// y pos, z neg
			OutTransform.SetRotation(FQuat(FVector(0.0f, 1.0f, 0.0f), UE_PI));
			//OutTransform.q = PxQuat(PxPi, PxVec3(0,1,0));
		}
	}
	else
	{
		if (InScale3D.Z > 0.f)
		{
			// y neg, z pos
			//OutTransform.q = PxQuat(PxPi, PxVec3(0,0,1));
			OutTransform.SetRotation(FQuat(FVector(0.0f, 0.0f, 1.0f), UE_PI));
		}
		else
		{
			// y neg, z neg
			//OutTransform.q = PxQuat(PxPi, PxVec3(1,0,0));
			OutTransform.SetRotation(FQuat(FVector(1.0f, 0.0f, 0.0f), UE_PI));
		}
	}

	// Use inverted mesh if determinant is negative
	return (InScale3D.X * InScale3D.Y * InScale3D.Z) < 0.f;
}


// TODO: Fixup types, these are more or less the PhysX types renamed as a temporary solution.
// Probably should move to different header as well.

inline constexpr uint32 AggregateMaxSize = 128;

class UPhysicalMaterial;
class UPrimitiveComponent;
struct FBodyInstance;
struct FConstraintInstance;
struct FKShapeElem;

/** Forward declarations */
struct FKShapeElem;
struct FCustomChaosPayload;
struct FChaosUserEntityAppend;
namespace Chaos
{
	struct FPhysicsObject;
}

namespace EChaosUserDataType
{
	enum Type
	{
		Invalid,
		BodyInstance,
		PhysicalMaterial,
		PhysScene,
		ConstraintInstance,
		PrimitiveComponent,
		AggShape,
		PhysicsObject,    // This will replace BodyInstances in the future
		ChaosUserEntity,  // This is used for adding custom user entities (unknown to UE)
		CustomPayload,	//This is intended for plugins
	};
};


struct FChaosUserData
{
protected:
	EChaosUserDataType::Type	Type;
	void*						Payload;

public:
	FChaosUserData()									:Type(EChaosUserDataType::Invalid), Payload(nullptr) {}
	FChaosUserData(FBodyInstance* InPayload)			:Type(EChaosUserDataType::BodyInstance), Payload(InPayload) {}
	FChaosUserData(UPhysicalMaterial* InPayload)		:Type(EChaosUserDataType::PhysicalMaterial), Payload(InPayload) {}
	FChaosUserData(FPhysScene* InPayload)			    :Type(EChaosUserDataType::PhysScene), Payload(InPayload) {}
	FChaosUserData(FConstraintInstance* InPayload)		:Type(EChaosUserDataType::ConstraintInstance), Payload(InPayload) {}
	FChaosUserData(UPrimitiveComponent* InPayload)		:Type(EChaosUserDataType::PrimitiveComponent), Payload(InPayload) {}
	FChaosUserData(FKShapeElem* InPayload)				:Type(EChaosUserDataType::AggShape), Payload(InPayload) {}
	FChaosUserData(Chaos::FPhysicsObject* InPayload) 	:Type(EChaosUserDataType::PhysicsObject), Payload(InPayload) {}
	FChaosUserData(FChaosUserEntityAppend* InPayload)	:Type(EChaosUserDataType::ChaosUserEntity), Payload(InPayload) {}
	FChaosUserData(FCustomChaosPayload* InPayload)		:Type(EChaosUserDataType::CustomPayload), Payload(InPayload) {}
	
	template <class T> static T* Get(void* UserData);
	template <class T> static void Set(void* UserData, T* Payload);

	//helper function to determine if userData is garbage (maybe dangling pointer)
	static bool IsGarbage(void* UserData){ return ((FChaosUserData*)UserData)->Type < EChaosUserDataType::Invalid || ((FChaosUserData*)UserData)->Type > EChaosUserDataType::CustomPayload; }

	template <EChaosUserDataType::Type UserDataType>
	bool Is() const { return Type == UserDataType; }
};

using FUserData = FChaosUserData;
using FPhysicsQueryFlag = FChaosQueryFlag;

template <> FORCEINLINE FBodyInstance* FChaosUserData::Get(void* UserData)			{ if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::BodyInstance) { return nullptr; } return (FBodyInstance*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE UPhysicalMaterial* FChaosUserData::Get(void* UserData)		{ if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::PhysicalMaterial) { return nullptr; } return (UPhysicalMaterial*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE FPhysScene* FChaosUserData::Get(void* UserData)				{ if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::PhysScene) { return nullptr; }return (FPhysScene*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE FConstraintInstance* FChaosUserData::Get(void* UserData)	{ if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::ConstraintInstance) { return nullptr; } return (FConstraintInstance*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE UPrimitiveComponent* FChaosUserData::Get(void* UserData)	{ if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::PrimitiveComponent) { return nullptr; } return (UPrimitiveComponent*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE FKShapeElem* FChaosUserData::Get(void* UserData)	{ if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::AggShape) { return nullptr; } return (FKShapeElem*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE Chaos::FPhysicsObject* FChaosUserData::Get(void* UserData) { if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::PhysicsObject) { return nullptr; } return (Chaos::FPhysicsObject*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE FChaosUserEntityAppend* FChaosUserData::Get(void* UserData) { if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::ChaosUserEntity) { return nullptr; } return (FChaosUserEntityAppend*)((FChaosUserData*)UserData)->Payload; }
template <> FORCEINLINE FCustomChaosPayload* FChaosUserData::Get(void* UserData) { if (!UserData || ((FChaosUserData*)UserData)->Type != EChaosUserDataType::CustomPayload) { return nullptr; } return (FCustomChaosPayload*)((FChaosUserData*)UserData)->Payload; }

template <> FORCEINLINE void FChaosUserData::Set(void* UserData, FBodyInstance* Payload)			{ check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::BodyInstance; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, UPhysicalMaterial* Payload)		{ check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::PhysicalMaterial; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, FPhysScene* Payload)				{ check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::PhysScene; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, FConstraintInstance* Payload)		{ check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::ConstraintInstance; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, UPrimitiveComponent* Payload)		{ check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::PrimitiveComponent; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, FKShapeElem* Payload)	{ check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::AggShape; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, Chaos::FPhysicsObject* Payload) { check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::PhysicsObject; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, FChaosUserEntityAppend* Payload) { check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::ChaosUserEntity; ((FChaosUserData*)UserData)->Payload = Payload; }
template <> FORCEINLINE void FChaosUserData::Set(void* UserData, FCustomChaosPayload* Payload) { check(UserData); ((FChaosUserData*)UserData)->Type = EChaosUserDataType::CustomPayload; ((FChaosUserData*)UserData)->Payload = Payload; }

struct UE_DEPRECATED(5.8, "Use Chaos::Filter::FQueryFilterData instead") FChaosFilterData
{
	FORCEINLINE FChaosFilterData()
	{
		word0 = word1 = word2 = word3 = 0;
	}

	FORCEINLINE FChaosFilterData(uint32 w0, uint32 w1, uint32 w2, uint32 w3) : word0(w0), word1(w1), word2(w2), word3(w3) {}

	FORCEINLINE void Reset()
	{
		*this = FChaosFilterData();
	}

	FORCEINLINE bool operator== (const FChaosFilterData& Other) const
	{
		return Other.word0 == word0 && Other.word1 == word1 && Other.word2 == word2 && Other.word3 == word3;
	}

	FORCEINLINE bool operator != (const FChaosFilterData& Other) const
	{
		return !(Other == *this);
	}

	uint32 word0;
	uint32 word1;
	uint32 word2;
	uint32 word3;
};

struct FChaosQueryFlag
{
	enum Enum
	{
		eSTATIC = (1 << 0),	//!< Traverse static shapes

		eDYNAMIC = (1 << 1),	//!< Traverse dynamic shapes

		ePREFILTER = (1 << 2),	//!< Run the pre-intersection-test filter

		ePOSTFILTER = (1 << 3),	//!< Run the post-intersection-test filter

		eANY_HIT = (1 << 4),	//!< Abort traversal as soon as any hit is found and return it via callback.block.
								//!< Helps query performance. Both eTOUCH and eBLOCK hitTypes are considered hits with this flag.

		eNO_BLOCK = (1 << 5),	//!< All hits are reported as touching. Overrides eBLOCK returned from user filters with eTOUCH.
								//!< This is also an optimization hint that may improve query performance.

		eSKIPNARROWPHASE = (1 << 6), //!< Skip narrow phase check for the query

		eRESERVED = (1 << 15)	//!< Reserved for internal use
	};
};

template <typename enumtype, typename storagetype = uint32_t>
class ChaosFlags
{
public:
	typedef storagetype InternalType;

	 FORCEINLINE ChaosFlags(void);
	 FORCEINLINE ChaosFlags(enumtype e);
	 FORCEINLINE ChaosFlags(const ChaosFlags<enumtype, storagetype>& f);
	 FORCEINLINE explicit ChaosFlags(storagetype b);

	 FORCEINLINE bool isSet(enumtype e) const;
	 FORCEINLINE ChaosFlags<enumtype, storagetype>& set(enumtype e);
	 FORCEINLINE bool operator==(enumtype e) const;
	 FORCEINLINE bool operator==(const ChaosFlags<enumtype, storagetype>& f) const;
	 FORCEINLINE bool operator==(bool b) const;
	 FORCEINLINE bool operator!=(enumtype e) const;
	 FORCEINLINE bool operator!=(const ChaosFlags<enumtype, storagetype>& f) const;

	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator=(const ChaosFlags<enumtype, storagetype>& f);
	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator=(enumtype e);

	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator|=(enumtype e);
	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator|=(const ChaosFlags<enumtype, storagetype>& f);
	 FORCEINLINE ChaosFlags<enumtype, storagetype> operator|(enumtype e) const;
	 FORCEINLINE ChaosFlags<enumtype, storagetype> operator|(const ChaosFlags<enumtype, storagetype>& f) const;

	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator&=(enumtype e);
	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator&=(const ChaosFlags<enumtype, storagetype>& f);
	 FORCEINLINE ChaosFlags<enumtype, storagetype> operator&(enumtype e) const;
	 FORCEINLINE ChaosFlags<enumtype, storagetype> operator&(const ChaosFlags<enumtype, storagetype>& f) const;

	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator^=(enumtype e);
	 FORCEINLINE ChaosFlags<enumtype, storagetype>& operator^=(const ChaosFlags<enumtype, storagetype>& f);
	 FORCEINLINE ChaosFlags<enumtype, storagetype> operator^(enumtype e) const;
	 FORCEINLINE ChaosFlags<enumtype, storagetype> operator^(const ChaosFlags<enumtype, storagetype>& f) const;

	 FORCEINLINE ChaosFlags<enumtype, storagetype> operator~(void) const;

	 FORCEINLINE operator bool(void) const;
	 FORCEINLINE operator uint8_t(void) const;
	 FORCEINLINE operator uint16_t(void) const;
	 FORCEINLINE operator uint32_t(void) const;

	 FORCEINLINE void clear(enumtype e);

public:
	friend FORCEINLINE ChaosFlags<enumtype, storagetype> operator&(enumtype a, ChaosFlags<enumtype, storagetype>& b)
	{
		ChaosFlags<enumtype, storagetype> out;
		out.mBits = a & b.mBits;
		return out;
	}

private:
	storagetype mBits;
};

typedef ChaosFlags<FChaosQueryFlag::Enum, uint16> FChaosQueryFlags;

inline FChaosQueryFlags U2CQueryFlags(FQueryFlags Flags)
{
	uint16 Result = 0;
	if (Flags & EQueryFlags::PreFilter)
	{
		Result |= FChaosQueryFlag::ePREFILTER;
	}

	if (Flags & EQueryFlags::PostFilter)
	{
		Result |= FChaosQueryFlag::ePOSTFILTER;
	}

	if (Flags & EQueryFlags::AnyHit)
	{
		Result |= FChaosQueryFlag::eANY_HIT;
	}

	if (Flags & EQueryFlags::SkipNarrowPhase)
	{
		Result |= FChaosQueryFlag::eSKIPNARROWPHASE;
	}
	
	return (FChaosQueryFlags)Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_DEPRECATED(5.8, "Use ChaosInterface::FSceneQueryCommonParams instead.") FChaosQueryFilterData
{
	explicit FORCEINLINE FChaosQueryFilterData() : flags(FChaosQueryFlag::eDYNAMIC | FChaosQueryFlag::eSTATIC), clientId(0) {}

	explicit FORCEINLINE FChaosQueryFilterData(const FChaosFilterData& fd, FChaosQueryFlags f) : data(fd), flags(f), clientId(0) {}

	explicit FORCEINLINE FChaosQueryFilterData(FChaosQueryFlags f) : flags(f), clientId(0) {}

	FChaosFilterData data;
	FChaosQueryFlags	flags;
	uint8 clientId;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::ChaosFlags(void)
{
	mBits = 0;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::ChaosFlags(enumtype e)
{
	mBits = static_cast<storagetype>(e);
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::ChaosFlags(const ChaosFlags<enumtype, storagetype>& f)
{
	mBits = f.mBits;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::ChaosFlags(storagetype b)
{
	mBits = b;
}

template <typename enumtype, typename storagetype>
FORCEINLINE bool ChaosFlags<enumtype, storagetype>::isSet(enumtype e) const
{
	return (mBits & static_cast<storagetype>(e)) == static_cast<storagetype>(e);
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::set(enumtype e)
{
	mBits = static_cast<storagetype>(e);
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE bool ChaosFlags<enumtype, storagetype>::operator==(enumtype e) const
{
	return mBits == static_cast<storagetype>(e);
}

template <typename enumtype, typename storagetype>
FORCEINLINE bool ChaosFlags<enumtype, storagetype>::operator==(const ChaosFlags<enumtype, storagetype>& f) const
{
	return mBits == f.mBits;
}

template <typename enumtype, typename storagetype>
FORCEINLINE bool ChaosFlags<enumtype, storagetype>::operator==(bool b) const
{
	return bool(*this) == b;
}

template <typename enumtype, typename storagetype>
FORCEINLINE bool ChaosFlags<enumtype, storagetype>::operator!=(enumtype e) const
{
	return mBits != static_cast<storagetype>(e);
}

template <typename enumtype, typename storagetype>
FORCEINLINE bool ChaosFlags<enumtype, storagetype>::operator!=(const ChaosFlags<enumtype, storagetype>& f) const
{
	return mBits != f.mBits;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::operator=(enumtype e)
{
	mBits = static_cast<storagetype>(e);
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::operator=(const ChaosFlags<enumtype, storagetype>& f)
{
	mBits = f.mBits;
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::operator|=(enumtype e)
{
	mBits |= static_cast<storagetype>(e);
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::
operator|=(const ChaosFlags<enumtype, storagetype>& f)
{
	mBits |= f.mBits;
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype> ChaosFlags<enumtype, storagetype>::operator|(enumtype e) const
{
	ChaosFlags<enumtype, storagetype> out(*this);
	out |= e;
	return out;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype> ChaosFlags<enumtype, storagetype>::
operator|(const ChaosFlags<enumtype, storagetype>& f) const
{
	ChaosFlags<enumtype, storagetype> out(*this);
	out |= f;
	return out;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::operator&=(enumtype e)
{
	mBits &= static_cast<storagetype>(e);
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::
operator&=(const ChaosFlags<enumtype, storagetype>& f)
{
	mBits &= f.mBits;
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype> ChaosFlags<enumtype, storagetype>::operator&(enumtype e) const
{
	ChaosFlags<enumtype, storagetype> out = *this;
	out.mBits &= static_cast<storagetype>(e);
	return out;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype> ChaosFlags<enumtype, storagetype>::
operator&(const ChaosFlags<enumtype, storagetype>& f) const
{
	ChaosFlags<enumtype, storagetype> out = *this;
	out.mBits &= f.mBits;
	return out;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::operator^=(enumtype e)
{
	mBits ^= static_cast<storagetype>(e);
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>& ChaosFlags<enumtype, storagetype>::
operator^=(const ChaosFlags<enumtype, storagetype>& f)
{
	mBits ^= f.mBits;
	return *this;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype> ChaosFlags<enumtype, storagetype>::operator^(enumtype e) const
{
	ChaosFlags<enumtype, storagetype> out = *this;
	out.mBits ^= static_cast<storagetype>(e);
	return out;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype> ChaosFlags<enumtype, storagetype>::
operator^(const ChaosFlags<enumtype, storagetype>& f) const
{
	ChaosFlags<enumtype, storagetype> out = *this;
	out.mBits ^= f.mBits;
	return out;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype> ChaosFlags<enumtype, storagetype>::operator~(void) const
{
	ChaosFlags<enumtype, storagetype> out;
	out.mBits = storagetype(~mBits);
	return out;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::operator bool(void) const
{
	return mBits ? true : false;
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::operator uint8_t(void) const
{
	return static_cast<uint8_t>(mBits);
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::operator uint16_t(void) const
{
	return static_cast<uint16_t>(mBits);
}

template <typename enumtype, typename storagetype>
FORCEINLINE ChaosFlags<enumtype, storagetype>::operator uint32_t(void) const
{
	return static_cast<uint32_t>(mBits);
}

template <typename enumtype, typename storagetype>
FORCEINLINE void ChaosFlags<enumtype, storagetype>::clear(enumtype e)
{
	mBits &= ~static_cast<storagetype>(e);
}
