// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/CollisionFilterData.h"
#include "Math/UnrealMathSSE.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceTypesCore.h"

namespace ChaosInterface { struct FPTLocationHit; }

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_INTERNAL PHYSICSCORE_API FCollisionFilterData C2UFilterData(const FChaosFilterData& FilterData);
UE_INTERNAL PHYSICSCORE_API FChaosFilterData U2CFilterData(const FCollisionFilterData& FilterData);

UE_INTERNAL PHYSICSCORE_API FCollisionFilterData ToUnrealFilterData(const FChaosFilterData& FilterData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//Find the face index for a given hit. This gives us a chance to modify face index based on things like most opposing normal

PHYSICSCORE_API uint32 FindFaceIndex(const FHitLocation& PHit, const FVector& UnitDirection);
PHYSICSCORE_API uint32 FindFaceIndex(const ChaosInterface::FPTLocationHit& PHit, const FVector& UnitDirection);
