// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Chaos/PhysicsObject.h"

class FChaosScene;

class PhysicsObjectPhysicsCoreInterface
{
public:
	static PHYSICSCORE_API FChaosScene* GetScene(TArrayView<const Chaos::FConstPhysicsObjectHandle> InObjects);
	static PHYSICSCORE_API FChaosScene* GetScene(const Chaos::FConstPhysicsObjectHandle InObject);
};
