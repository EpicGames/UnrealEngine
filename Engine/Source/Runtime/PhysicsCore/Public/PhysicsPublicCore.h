// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UPhysicalMaterial;

/** Set of delegates to allowing hooking different parts of the physics engine */
class FPhysicsDelegatesCore
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdatePhysXMaterial, UPhysicalMaterial*);
	static PHYSICSCORE_API FOnUpdatePhysXMaterial OnUpdatePhysXMaterial;
};
