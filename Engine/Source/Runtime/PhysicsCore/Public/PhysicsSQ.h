// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Physics SQ module interface
 */
class IPhysicsSQModule : public IModuleInterface
{
public:
};
