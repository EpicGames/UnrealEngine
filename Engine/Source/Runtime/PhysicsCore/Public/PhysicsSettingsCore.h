// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "PhysicsSettingsEnums.h"
#include "BodySetupEnums.h"
#include "ChaosSolverConfiguration.h"

#include "PhysicsSettingsCore.generated.h"


/**
 * Default physics settings.
 */
UCLASS(config=Engine,defaultconfig,meta=(DisplayName="Physics"), MinimalAPI)
class UPhysicsSettingsCore: public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	/** Default gravity. */
	UPROPERTY(config,EditAnywhere,Category = Constants)
	float DefaultGravityZ;

	/** Default terminal velocity for Physics Volumes. */
	UPROPERTY(config,EditAnywhere,Category = Constants)
	float DefaultTerminalVelocity;

	/** Default fluid friction for Physics Volumes. */
	UPROPERTY(config,EditAnywhere,Category = Constants)
	float DefaultFluidFriction;

	/** Amount of memory to reserve for PhysX simulate(), this is per pxscene and will be rounded up to the next 16K boundary */
	UPROPERTY(config,EditAnywhere,Category = Constants,meta = (ClampMin = "0",UIMin = "0"))
	int32 SimulateScratchMemorySize;

	/** Threshold for ragdoll bodies above which they will be added to an aggregate before being added to the scene */
	UPROPERTY(config,EditAnywhere,meta = (ClampMin = "1",UIMin = "1",ClampMax = "127",UIMax = "127"),Category = Constants)
	int32 RagdollAggregateThreshold;

	/** Triangles from triangle meshes (BSP) with an area less than or equal to this value will be removed from physics collision data. Set to less than 0 to disable. */
	UPROPERTY(config,EditAnywhere,AdvancedDisplay,meta = (ClampMin = "-1.0",UIMin = "-1.0",ClampMax = "10.0",UIMax = "10.0"),Category = Constants)
	float TriangleMeshTriangleMinAreaThreshold;

	/** If set to true, the scene will use enhanced determinism at the cost of a bit more resources. See eENABLE_ENHANCED_DETERMINISM to learn about the specifics */
	UPROPERTY(config,EditAnywhere,Category = Simulation)
	bool bEnableEnhancedDeterminism;

	/** Enables shape sharing between sync and async scene for static rigid actors */
	UPROPERTY(config,EditAnywhere,AdvancedDisplay,Category = Simulation)
	bool bEnableShapeSharing;

	/** Enables persistent contact manifolds. This will generate fewer contact points, but with more accuracy. Reduces stability of stacking, but can help energy conservation.*/
	UPROPERTY(config,EditAnywhere,AdvancedDisplay,Category = Simulation)
	bool bEnablePCM;

	/** Enables stabilization of contacts for slow moving bodies. This will help improve the stability of stacking.*/
	UPROPERTY(config,EditAnywhere,AdvancedDisplay,Category = Simulation)
	bool bEnableStabilization;

	/** Whether to warn when physics locks are used incorrectly. Turning this off is not recommended and should only be used by very advanced users. */
	UPROPERTY(config,EditAnywhere,AdvancedDisplay,Category = Simulation)
	bool bWarnMissingLocks;

	/** Can 2D physics be used (Box2D)? */
	UPROPERTY(config,EditAnywhere,Category = Simulation)
	bool bEnable2DPhysics;

	/**
	*  If true, static meshes will use per poly collision as complex collision by default. If false the default behavior is the same as UseSimpleAsComplex. */
	UPROPERTY(config)
	bool bDefaultHasComplexCollision_DEPRECATED;

	
	/** Minimum relative velocity required for an object to bounce. A typical value for simulation stability is about 0.2 * gravity*/
	UPROPERTY(config, EditAnywhere, Category = Simulation, meta = (ClampMin = "0", UIMin = "0"))
	float BounceThresholdVelocity;

	/** Friction combine mode, controls how friction is computed for multiple materials. */
	UPROPERTY(config, EditAnywhere, Category=Simulation)
	TEnumAsByte<EFrictionCombineMode::Type> FrictionCombineMode;

	/** Restitution combine mode, controls how restitution is computed for multiple materials. */
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	TEnumAsByte<EFrictionCombineMode::Type> RestitutionCombineMode;

	/** Max angular velocity that a simulated object can achieve.*/
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	float MaxAngularVelocity;

	/** Max velocity which may be used to depenetrate simulated physics objects. 0 means no maximum. */
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	float MaxDepenetrationVelocity;

	/** Contact offset multiplier. When creating a physics shape we look at its bounding volume and multiply its minimum value by this multiplier. A bigger number will generate contact points earlier which results in higher stability at the cost of performance. */
	UPROPERTY(config, EditAnywhere, Category = Simulation, meta = (ClampMin = "0.001", UIMin = "0.001"))
	float ContactOffsetMultiplier;

	/** Min Contact offset. */
	UPROPERTY(config, EditAnywhere, Category = Simulation, meta = (ClampMin = "0.0001", UIMin = "0.0001"))
	float MinContactOffset;

	/** Max Contact offset. */
	UPROPERTY(config, EditAnywhere, Category = Simulation, meta = (ClampMin = "0.001", UIMin = "0.001"))
	float MaxContactOffset;

	/**
	*  If true, simulate physics for this component on a dedicated server.
	*  This should be set if simulating physics and replicating with a dedicated server.
	*/
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	bool bSimulateSkeletalMeshOnDedicatedServer;

	/**
	*  Determines the default physics shape complexity. */
	UPROPERTY(config, EditAnywhere, Category = Simulation)
	TEnumAsByte<ECollisionTraceFlag> DefaultShapeComplexity;

	/** Options to apply to Chaos solvers on creation */
	UPROPERTY(config, EditAnywhere, Category = ChaosPhysics)
	FChaosSolverConfiguration SolverOptions;

	static PHYSICSCORE_API UPhysicsSettingsCore* Get();

	PHYSICSCORE_API virtual void PostInitProperties() override;

protected:
	static PHYSICSCORE_API void SetDefaultSettings(UPhysicsSettingsCore* InSettings);

private:
	// Override default settings.
	// This should be set up to point to the CDO of the leaf settings class (as edited in Project Settings)
	static UPhysicsSettingsCore* DefaultSettings;
};
