// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsSettingsEnums.h: Declares the PhysicsSettings enums
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsSettingsEnums.generated.h"

UENUM()
namespace EFrictionCombineMode
{
	enum Type : int
	{
		/** Uses the average value of the materials touching: (a+b)/2 */
		Average = 0,	
		/** Uses the minimum value of the materials touching: min(a,b) */
		Min = 1,		
		/** Uses the product of the values of the materials touching: a*b */
		Multiply = 2,	
		/** Uses the maximum value of materials touching: max(a,b) */
		Max = 3
	};
}
