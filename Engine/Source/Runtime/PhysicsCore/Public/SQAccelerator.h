// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Real.h"
#include "ChaosInterfaceWrapperCore.h"
#include "ChaosSQTypes.h"
#include "Containers/Array.h"
#include "Math/BoxSphereBounds.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceWrapperShared.h"
#include "SceneQueryCommonParams.h"

namespace ChaosInterface { struct FOverlapHit; }
namespace ChaosInterface { struct FPTOverlapHit; }
namespace ChaosInterface { struct FPTRaycastHit; }
namespace ChaosInterface { struct FPTSweepHit; }
namespace ChaosInterface { struct FRaycastHit; }
namespace ChaosInterface { struct FSweepHit; }
namespace ChaosInterface { template <typename HitType> class FSQHitBuffer; }

namespace Chaos
{
	class FAccelerationStructureHandle;
	class FImplicitObject;
	template <typename TPayload, typename T, int d>
	class ISpatialAcceleration;
}

class FSQAccelerator;
class ICollisionQueryFilterCallbackBase;
struct FCollisionFilterData;
struct FCollisionQueryParams;
struct FCollisionQueryParams;

class FChaosSQAccelerator
{
public:

	PHYSICSCORE_API FChaosSQAccelerator(const Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle,Chaos::FReal, 3>& InSpatialAcceleration);
	virtual ~FChaosSQAccelerator() = default;

	PHYSICSCORE_API void Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const;
	PHYSICSCORE_API void Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const;

	PHYSICSCORE_API void Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const;
	PHYSICSCORE_API void Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const;

	PHYSICSCORE_API void Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const;
	PHYSICSCORE_API void Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTOverlapHit>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const;


	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
	PHYSICSCORE_API void Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {}) const;
	UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
	PHYSICSCORE_API void Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {}) const;

	UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
	PHYSICSCORE_API void Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {}) const;
	UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
	PHYSICSCORE_API void Sweep(const Chaos::FImplicitObject& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {}) const;

	UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
	PHYSICSCORE_API void Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {}) const;
	UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
	PHYSICSCORE_API void Overlap(const Chaos::FImplicitObject& QueryGeom, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<ChaosInterface::FPTOverlapHit>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {}) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	const Chaos::ISpatialAcceleration<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>& SpatialAcceleration;

	template <typename TRaycastHit>
	void RaycastImp(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<TRaycastHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams) const;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// An interface to the scene query accelerator that allows us to run queries against either PhysX or Chaos
// This was used in the 2019 GDC demos and is now broken. To make it work again, we would need to implement
// the FChaosSQAcceleratorAdapter below to use its internal SQ accelerator and convert the inputs and outputs
// from/to PhysX types.
class UE_DEPRECATED(5.8, "This is unused and will be removed") ISQAccelerator
{
public:
	virtual ~ISQAccelerator() = default;
	virtual void Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, FPhysicsHitCallback<FHitRaycast>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const = 0;
	virtual void Sweep(const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, FPhysicsHitCallback<FHitSweep>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const = 0;
	virtual void Overlap(const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<FHitOverlap>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const = 0;
};

class UE_DEPRECATED(5.8, "This is unused and will be removed") FSQAcceleratorUnion : public ISQAccelerator
{
public:

	PHYSICSCORE_API virtual void Raycast(const FVector& Start, const FVector& Dir, const float DeltaMagnitude, FPhysicsHitCallback<FHitRaycast>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const override;
	PHYSICSCORE_API virtual void Sweep(const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, FPhysicsHitCallback<FHitSweep>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const override;
	PHYSICSCORE_API virtual void Overlap(const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<FHitOverlap>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback) const override;

	PHYSICSCORE_API void AddSQAccelerator(ISQAccelerator* InAccelerator);
	PHYSICSCORE_API void RemoveSQAccelerator(ISQAccelerator* AcceleratorToRemove);

private:
	TArray<ISQAccelerator*> Accelerators;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
