// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "ChaosInterfaceWrapperCore.h"
#include "ChaosSQTypes.h"
#include "CollisionQueryFilterCallbackCore.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "PhysXInterfaceWrapperCore.h"
#include "PhysXPublicCore.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceWrapperShared.h"
#include "Templates/UniquePtr.h"

class FPhysTestSerializer;
struct FCollisionFilterData;

namespace ChaosInterface
{
	struct FOverlapHit;
	struct FRaycastHit;
	struct FSweepHit;
}

namespace Chaos
{
	class FChaosArchive;
	class FImplicitObject;
	class FPerShapeData;
}

//Allows us to capture a scene query with either physx or chaos and then load it into either format for testing purposes
PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_INTERNAL FSQCapture
{
	PHYSICSCORE_API ~FSQCapture();
	FSQCapture(const FSQCapture&) = delete;
	FSQCapture& operator=(const FSQCapture&) = delete;

	enum class ESQType : uint8
	{
		Raycast,
		Sweep,
		Overlap
	} SQType;

	PHYSICSCORE_API void StartCaptureChaosSweep(const Chaos::FPBDRigidsEvolution& Evolution, const Chaos::FImplicitObject& InQueryGeom, const FTransform& InStartTM, const FVector& InDir, float InDeltaMag, FHitFlags InOutputFlags, const ChaosInterface::FQueryFilterData& QueryFilter, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback);
	PHYSICSCORE_API void EndCaptureChaosSweep(const ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>& Results);

	PHYSICSCORE_API void StartCaptureChaosRaycast(const Chaos::FPBDRigidsEvolution& Evolution, const FVector& InStartPoint, const FVector& InDir, float InDeltaMag, FHitFlags InOutputFlags, const ChaosInterface::FQueryFilterData& QueryFilter, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback);
	PHYSICSCORE_API void EndCaptureChaosRaycast(const ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>& Results);

	PHYSICSCORE_API void StartCaptureChaosOverlap(const Chaos::FPBDRigidsEvolution& Evolution, const Chaos::FImplicitObject& InQueryGeom, const FTransform& InStartTM, const ChaosInterface::FQueryFilterData& QueryFilter, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback);
	PHYSICSCORE_API void EndCaptureChaosOverlap(const ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>& Results);

	PHYSICSCORE_API ECollisionQueryHitType GetFilterResult(const Chaos::FPerShapeData* Shape, const Chaos::FGeometryParticle* Actor) const;
	
	FVector Dir;
	FTransform StartTM;	//only valid if overlap or sweep
	FVector StartPoint;	//only valid if raycast

	float DeltaMag;
	FHitFlags OutputFlags;
	ChaosInterface::FQueryFilterData QueryFilterData;
	TUniquePtr<ICollisionQueryFilterCallbackBase> FilterCallback;

	Chaos::FImplicitObjectPtr ChaosImplicitGeometry;
	
	ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit> ChaosSweepBuffer;
	TArray<ChaosInterface::FSweepHit> ChaosSweepTouches;

	ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit> ChaosRaycastBuffer;
	TArray<ChaosInterface::FRaycastHit> ChaosRaycastTouches;

	ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit> ChaosOverlapBuffer;
	TArray<ChaosInterface::FOverlapHit> ChaosOverlapTouches;

	UE_DEPRECATED(5.4, "Please use ChaosImplicitGeometry instead.")
    TUniquePtr<Chaos::FImplicitObject> ChaosOwnerObject;	//should be private, do not access directly
    
    UE_DEPRECATED(5.4, "Please use ChaosImplicitGeometry instead.")
    const Chaos::FImplicitObject* ChaosGeometry;
    
    UE_DEPRECATED(5.4, "Please use ChaosImplicitGeometry instead.")
    TUniquePtr<Chaos::FImplicitObject> SerializableChaosGeometry;

private:
	PHYSICSCORE_API FSQCapture(FPhysTestSerializer& OwningPhysSerializer);	//This should be created by PhysTestSerializer
	PHYSICSCORE_API void Serialize(Chaos::FChaosArchive& Ar);

	friend FPhysTestSerializer;

	TArray<uint8> GeomData;
	TArray<uint8> HitData;

	FPhysTestSerializer& PhysSerializer;

	PHYSICSCORE_API void CaptureChaosFilterResults(const Chaos::FPBDRigidsEvolution& Evolution, const FCollisionFilterData& FilterData, ICollisionQueryFilterCallbackBase& Callback);

	TMap<Chaos::FGeometryParticle*, TArray<TPair<Chaos::FPerShapeData*, ECollisionQueryHitType>>> ChaosActorToShapeHitsArray;

	template <typename THit>
	void SerializeChaosBuffers(Chaos::FChaosArchive& Ar, int32 Version, ChaosInterface::FSQHitBuffer<THit>& ChaosBuffer);

	PHYSICSCORE_API void SerializeChaosActorToShapeHitsArray(Chaos::FChaosArchive& Ar);

	bool bDiskDataIsChaos;
	bool bChaosDataReady;
	bool bPhysXDataReady;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
