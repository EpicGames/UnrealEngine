// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

UE_DEPRECATED_HEADER(5.8, "This file is unused and will be removed")  

#include "PhysicsCore.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "PhysicsInterfaceUtilsCore.h"
#include "PhysTestSerializer.h"
#include "SQAccelerator.h"

#ifndef SQ_REPLAY_TEST
#define SQ_REPLAY_TEST(cond) bEnsureOnMismatch ? ensure(cond) : (cond)
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <bool bHasPhysX = false>
UE_DEPRECATED(5.8, "This is unused and will be removed")
void SQPerfComparisonHelper(const FString& TestName, FPhysTestSerializer& Serializer, bool bEnsureOnMismatch = false)
{
	using namespace Chaos;
	uint32 PhysXSum = 0;
	uint32 ChaosSum = 0;
	double NumIterations = bHasPhysX ? 100 : 10000;
	FPendingSpatialDataQueue Empty;
	//double NumIterations = 1000000;

	const FSQCapture& CapturedSQ = *Serializer.GetSQCapture();
	switch (CapturedSQ.SQType)
	{
	case FSQCapture::ESQType::Raycast:
	{
		ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* Accelerator = nullptr;
		Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
		FChaosSQAccelerator SQAccelerator(*Accelerator);
		for (double i = 0; i < NumIterations; ++i)
		{
			auto ChaosHitBuffer = MakeUnique<ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>>();
			uint32 StartTime = FPlatformTime::Cycles();
			SQAccelerator.Raycast(CapturedSQ.StartPoint, CapturedSQ.Dir, CapturedSQ.DeltaMag, *ChaosHitBuffer, CapturedSQ.OutputFlags.HitFlags, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
			ChaosSum += FPlatformTime::Cycles() - StartTime;
		}
		break;
	}
	case FSQCapture::ESQType::Sweep:
	{
		ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* Accelerator = nullptr;
		Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
		FChaosSQAccelerator SQAccelerator(*Accelerator);
		for (double i = 0; i < NumIterations; ++i)
		{
			auto ChaosHitBuffer = MakeUnique<ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>>();
			uint32 StartTime = FPlatformTime::Cycles();
			SQAccelerator.Sweep(*CapturedSQ.ChaosImplicitGeometry, CapturedSQ.StartTM, CapturedSQ.Dir, CapturedSQ.DeltaMag, *ChaosHitBuffer, CapturedSQ.OutputFlags.HitFlags, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
			ChaosSum += FPlatformTime::Cycles() - StartTime;
		}
		break;
	}
	case FSQCapture::ESQType::Overlap:
	{
		ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* Accelerator = nullptr;
		Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
		FChaosSQAccelerator SQAccelerator(*Accelerator);
		for (double i = 0; i < NumIterations; ++i)
		{
			auto ChaosHitBuffer = MakeUnique<ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>>();
			uint32 StartTime = FPlatformTime::Cycles();
			SQAccelerator.Overlap(*CapturedSQ.ChaosImplicitGeometry, CapturedSQ.StartTM, *ChaosHitBuffer, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
			ChaosSum += FPlatformTime::Cycles() - StartTime;
		}
		break;
	}
	}

	float ChaosMs = FPlatformTime::ToMilliseconds(ChaosSum);
	float ChaosAvgUs = (ChaosMs * 1000) / NumIterations;

	float PhysXMs = FPlatformTime::ToMilliseconds(PhysXSum);
	float PhysXAvgUs = (PhysXMs * 1000) / NumIterations;


	if (bHasPhysX)
	{
		UE_LOGF(LogPhysicsCore, Warning, "Perf Test:%ls\nPhysX:%f(us), Chaos:%f(us)", *TestName, PhysXAvgUs, ChaosAvgUs);
	}
	else
	{
		UE_LOGF(LogPhysicsCore, Warning, "Perf Test:%ls\nChaos:%f(us), Total:%f(ms)", *TestName, ChaosAvgUs, ChaosMs);
		//UE_LOGF(LogPhysicsCore, Warning, "Perf Test:%ls\nChaos:%f(us)", *TestName, AvgChaos);
	}
}


#if 0
UE_DEPRECATED(5.8, "This is unused and will be removed")
bool SQComparisonHelper(FPhysTestSerializer& Serializer, bool bEnsureOnMismatch = false)
{
	using namespace Chaos;

	bool bTestPassed = true;
	const float DistanceTolerance = 1e-1f;
	const float NormalTolerance = 1e-2f;
	FPendingSpatialDataQueue Empty;

	const FSQCapture& CapturedSQ = *Serializer.GetSQCapture();
	switch (CapturedSQ.SQType)
	{
	case FSQCapture::ESQType::Raycast:
	{
		auto PxHitBuffer = MakeUnique<PhysXInterface::FDynamicHitBuffer<PxRaycastHit>>();
		Serializer.GetPhysXData()->raycast(U2PVector(CapturedSQ.StartPoint), U2PVector(CapturedSQ.Dir), CapturedSQ.DeltaMag, *PxHitBuffer, U2PHitFlags(CapturedSQ.OutputFlags.HitFlags), CapturedSQ.QueryFilterData, CapturedSQ.FilterCallback.Get());

		bTestPassed &= SQ_REPLAY_TEST(PxHitBuffer->hasBlock == CapturedSQ.PhysXRaycastBuffer.hasBlock);
		bTestPassed &= SQ_REPLAY_TEST(PxHitBuffer->GetNumHits() == CapturedSQ.PhysXRaycastBuffer.GetNumHits());
		for (int32 Idx = 0; Idx < PxHitBuffer->GetNumHits(); ++Idx)
		{
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer.GetHits()[Idx].normal.x, CapturedSQ.PhysXRaycastBuffer.GetHits()[Idx].normal.x));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer.GetHits()[Idx].normal.y, CapturedSQ.PhysXRaycastBuffer.GetHits()[Idx].normal.y));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer.GetHits()[Idx].normal.z, CapturedSQ.PhysXRaycastBuffer.GetHits()[Idx].normal.z));
		}

		if (PxHitBuffer->hasBlock)
		{
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer->block.position.x, CapturedSQ.PhysXRaycastBuffer.block.position.x, DistanceTolerance));
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer->block.position.y, CapturedSQ.PhysXRaycastBuffer.block.position.y, DistanceTolerance));
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer->block.position.z, CapturedSQ.PhysXRaycastBuffer.block.position.z, DistanceTolerance));
		}

		auto ChaosHitBuffer = MakeUnique<ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit>>();
		ISpatialAccelerationCollection<FAccelerationStructureHandle, float, 3>* Accelerator = nullptr;
		Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
		FChaosSQAccelerator SQAccelerator(*Accelerator); SQAccelerator.Raycast(CapturedSQ.StartPoint, CapturedSQ.Dir, CapturedSQ.DeltaMag, *ChaosHitBuffer, CapturedSQ.OutputFlags.HitFlags, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
		
		bTestPassed &= SQ_REPLAY_TEST(ChaosHitBuffer->HasBlockingHit() == CapturedSQ.PhysXRaycastBuffer.hasBlock);
		bTestPassed &= SQ_REPLAY_TEST(ChaosHitBuffer->GetNumHits() == CapturedSQ.PhysXRaycastBuffer.GetNumHits());
		for (int32 Idx = 0; Idx < ChaosHitBuffer->GetNumHits(); ++Idx)
		{
			//not sorted
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer.GetHits()[Idx].WorldNormal.X, CapturedSQ.PhysXRaycastBuffer.GetHits()[Idx].normal.x));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer.GetHits()[Idx].WorldNormal.Y, CapturedSQ.PhysXRaycastBuffer.GetHits()[Idx].normal.y));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer.GetHits()[Idx].WorldNormal.Z, CapturedSQ.PhysXRaycastBuffer.GetHits()[Idx].normal.z));
		}

		if (ChaosHitBuffer->HasBlockingHit())
		{
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer->GetBlock()->WorldPosition.X, CapturedSQ.PhysXRaycastBuffer.block.position.x, DistanceTolerance));
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer->GetBlock()->WorldPosition.Y, CapturedSQ.PhysXRaycastBuffer.block.position.y, DistanceTolerance));
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer->GetBlock()->WorldPosition.Z, CapturedSQ.PhysXRaycastBuffer.block.position.z, DistanceTolerance));

			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer->GetBlock()->WorldNormal.X, CapturedSQ.PhysXRaycastBuffer.block.normal.x, NormalTolerance));
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer->GetBlock()->WorldNormal.Y, CapturedSQ.PhysXRaycastBuffer.block.normal.y, NormalTolerance));
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer->GetBlock()->WorldNormal.Z, CapturedSQ.PhysXRaycastBuffer.block.normal.z, NormalTolerance));
		}
		break;
	}
	case FSQCapture::ESQType::Sweep:
	{
		//For sweep there are many solutions (many contacts possible) so we only bother testing for Distance
		auto PxHitBuffer = MakeUnique<PhysXInterface::FDynamicHitBuffer<PxSweepHit>>();
		Serializer.GetPhysXData()->sweep(CapturedSQ.PhysXGeometry.any(), U2PTransform(CapturedSQ.StartTM), U2PVector(CapturedSQ.Dir), CapturedSQ.DeltaMag, *PxHitBuffer, U2PHitFlags(CapturedSQ.OutputFlags.HitFlags), CapturedSQ.QueryFilterData, CapturedSQ.FilterCallback.Get());

		bTestPassed &= SQ_REPLAY_TEST(PxHitBuffer->hasBlock == CapturedSQ.PhysXSweepBuffer.hasBlock);
		bTestPassed &= SQ_REPLAY_TEST(PxHitBuffer->GetNumHits() == CapturedSQ.PhysXSweepBuffer.GetNumHits());
		for (int32 Idx = 0; Idx < PxHitBuffer->GetNumHits(); ++Idx)
		{
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer.GetHits()[Idx].normal.x, CapturedSQ.PhysXSweepBuffer.GetHits()[Idx].normal.x, DistanceTolerance));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer.GetHits()[Idx].normal.y, CapturedSQ.PhysXSweepBuffer.GetHits()[Idx].normal.y, DistanceTolerance));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(PxHitBuffer.GetHits()[Idx].normal.z, CapturedSQ.PhysXSweepBuffer.GetHits()[Idx].normal.z, DistanceTolerance));
		}

		auto ChaosHitBuffer = MakeUnique<ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit>>();
		ISpatialAccelerationCollection<FAccelerationStructureHandle, float, 3>* Accelerator = nullptr;
		Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
		FChaosSQAccelerator SQAccelerator(*Accelerator); SQAccelerator.Sweep(*CapturedSQ.ChaosGeometry, CapturedSQ.StartTM, CapturedSQ.Dir, CapturedSQ.DeltaMag, *ChaosHitBuffer, CapturedSQ.OutputFlags.HitFlags, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
		
		bTestPassed &= SQ_REPLAY_TEST(ChaosHitBuffer->HasBlockingHit() == CapturedSQ.PhysXSweepBuffer.hasBlock);
		bTestPassed &= SQ_REPLAY_TEST(ChaosHitBuffer->GetNumHits() == CapturedSQ.PhysXSweepBuffer.GetNumHits());
		for (int32 Idx = 0; Idx < ChaosHitBuffer->GetNumHits(); ++Idx)
		{
			//not sorted
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer.GetHits()[Idx].WorldNormal.X, CapturedSQ.PhysXSweepBuffer.GetHits()[Idx].normal.x));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer.GetHits()[Idx].WorldNormal.Y, CapturedSQ.PhysXSweepBuffer.GetHits()[Idx].normal.y));
			//bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer.GetHits()[Idx].WorldNormal.Z, CapturedSQ.PhysXSweepBuffer.GetHits()[Idx].normal.z));
		}

		if (ChaosHitBuffer->HasBlockingHit())
		{
			bTestPassed &= SQ_REPLAY_TEST(FMath::IsNearlyEqual(ChaosHitBuffer->GetBlock()->Distance, CapturedSQ.PhysXSweepBuffer.block.distance, DistanceTolerance));
		}
		break;
	}
	case FSQCapture::ESQType::Overlap:
	{
		auto PxHitBuffer = MakeUnique<PhysXInterface::FDynamicHitBuffer<PxOverlapHit>>();
		Serializer.GetPhysXData()->overlap(CapturedSQ.PhysXGeometry.any(), U2PTransform(CapturedSQ.StartTM), *PxHitBuffer, CapturedSQ.QueryFilterData, CapturedSQ.FilterCallback.Get());

		bTestPassed &= SQ_REPLAY_TEST(PxHitBuffer->GetNumHits() == CapturedSQ.PhysXOverlapBuffer.GetNumHits());

		auto ChaosHitBuffer = MakeUnique<ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit>>();
		ISpatialAccelerationCollection<FAccelerationStructureHandle, float, 3>* Accelerator = nullptr;
		Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
		FChaosSQAccelerator SQAccelerator(*Accelerator);
		SQAccelerator.Overlap(*CapturedSQ.ChaosGeometry, CapturedSQ.StartTM, *ChaosHitBuffer, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
		
		bTestPassed &= SQ_REPLAY_TEST(ChaosHitBuffer->GetNumHits() == CapturedSQ.PhysXOverlapBuffer.GetNumHits());
		break;
	}
	}

	return bTestPassed;
}

#endif

UE_DEPRECATED(5.8, "This is unused and will be removed")
bool SQValidityHelper(FPhysTestSerializer& Serializer)
{
	using namespace Chaos;

	bool bTestPassed = true;
	const float DistanceTolerance = 1e-1f;
	const float NormalTolerance = 1e-2f;
	FPendingSpatialDataQueue Empty;

	const FSQCapture& CapturedSQ = *Serializer.GetSQCapture();
	switch (CapturedSQ.SQType)
	{
		case FSQCapture::ESQType::Raycast:
		{
			ChaosInterface::FSQHitBuffer<ChaosInterface::FRaycastHit> ChaosHitBuffer;
			ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* Accelerator = nullptr;
			
			Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
			FChaosSQAccelerator SQAccelerator(*Accelerator);
			SQAccelerator.Raycast(CapturedSQ.StartPoint, CapturedSQ.Dir, CapturedSQ.DeltaMag, ChaosHitBuffer, CapturedSQ.OutputFlags.HitFlags, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
			
			const bool bHasBlockingHit = ChaosHitBuffer.HasBlockingHit();
			const int32 NumHits = ChaosHitBuffer.GetNumHits();
			for (int32 Idx = 0; Idx < NumHits; ++Idx)
			{
				// TODO: DO tests
			}
			break;
		}
		case FSQCapture::ESQType::Sweep:
		{
			ChaosInterface::FSQHitBuffer<ChaosInterface::FSweepHit> ChaosHitBuffer;
			ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* Accelerator = nullptr;
			Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
			FChaosSQAccelerator SQAccelerator(*Accelerator);
			SQAccelerator.Sweep(*CapturedSQ.ChaosImplicitGeometry, CapturedSQ.StartTM, CapturedSQ.Dir, CapturedSQ.DeltaMag, ChaosHitBuffer, CapturedSQ.OutputFlags.HitFlags, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
			
			const bool bHasBlockingHit = ChaosHitBuffer.HasBlockingHit();
			const int32 NumHits = ChaosHitBuffer.GetNumHits();
			for (int32 Idx = 0; Idx < NumHits; ++Idx)
			{
				ChaosInterface::FSweepHit& Hit = ChaosHitBuffer.GetHits()[Idx];

				if (!HadInitialOverlap(Hit))
				{
					const int32 FaceIdx = FindFaceIndex(Hit, CapturedSQ.Dir);
					bTestPassed |= (FaceIdx != INDEX_NONE);
				}
			}

			break;
		}
		case FSQCapture::ESQType::Overlap:
		{
			ChaosInterface::FSQHitBuffer<ChaosInterface::FOverlapHit> ChaosHitBuffer;
			ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* Accelerator = nullptr;
			Serializer.GetChaosData()->UpdateExternalAccelerationStructure_External(Accelerator, Empty);
			FChaosSQAccelerator SQAccelerator(*Accelerator);
			SQAccelerator.Overlap(*CapturedSQ.ChaosImplicitGeometry, CapturedSQ.StartTM, ChaosHitBuffer, CapturedSQ.QueryFilterData, *CapturedSQ.FilterCallback);
			break;
		}
	}

	return bTestPassed;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

