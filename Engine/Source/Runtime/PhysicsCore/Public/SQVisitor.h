// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/DebugDrawQueue.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/Interface/SQTypes.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/ParticleHandle.h"
#include "ChaosInterfaceWrapperCore.h"
#include "CollisionQueryFilterCallbackCore.h"
#include "PhysicsInterfaceTypesCore.h"
#include "PhysicsInterfaceUtilsCore.h"
#include "SceneQueryCommonParams.h"

#include <type_traits>

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#endif

#include "ChaosVDSQVisitorHelpers.h"

#if CHAOS_DEBUG_DRAW
extern PHYSICSCORE_API int32 ChaosSQDrawDebugVisitorQueries;
extern PHYSICSCORE_API FAutoConsoleVariableRef CVarChaosSQDrawDebugQueries;
#endif

template <typename TLocationHit>
void FillHitHelper(TLocationHit& Hit, const Chaos::FReal Distance, const FVector& WorldPosition, const FVector& WorldNormal, int32 FaceIdx, bool bComputeMTD)
{
	if constexpr (std::is_base_of_v<ChaosInterface::FLocationHit, TLocationHit> || std::is_base_of_v<ChaosInterface::FPTLocationHit, TLocationHit>)
	{
		Hit.Distance = Distance;
		Hit.WorldPosition = WorldPosition;
		Hit.WorldNormal = WorldNormal;
		Hit.Flags = Distance > 0.f || bComputeMTD ? EHitFlags::Distance | EHitFlags::Normal | EHitFlags::Position : EHitFlags::Distance | EHitFlags::FaceIndex;
		Hit.FaceIndex = FaceIdx;
	}
}

template <typename QueryGeometryType, typename TPayload, typename THitType, bool bGTData = true>
struct TSQVisitor : public Chaos::ISpatialVisitor<TPayload, Chaos::FReal>
{
	using TGeometryType = std::conditional_t<bGTData, Chaos::FGeometryParticle, Chaos::FGeometryParticleHandle>;
	using FAABB3 = Chaos::FAABB3;
	using FVec3 = Chaos::FVec3;
	TSQVisitor(const FVector& InStartPoint, const FVector& InDir, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer, EHitFlags InOutputFlags,
		const ChaosInterface::FSceneQueryCommonParams& InCommonParams)
		: StartPoint(InStartPoint)
		, Dir(InDir)
		, OutputFlags(InOutputFlags)
		, bAnyHit(false)
		, DebugParams(InCommonParams.DebugParams)
		, HitFaceNormal(Chaos::FVec3::ZeroVector)
		, HitBuffer(InHitBuffer)
		, QueryFlags(InCommonParams.Flags)
		, QueryFilterData(InCommonParams.FilterData)
		, QueryCallback(InCommonParams.GetQueryCallback())
		, QueryCenter(StartTM.GetLocation())
	{
		QueryGeomWorldBounds = FAABB3(QueryCenter, QueryCenter);
		bAnyHit = QueryFlags & FChaosQueryFlag::eANY_HIT;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use the overload that takes a ChaosInterface::FSceneQueryCommonParams instead")
	TSQVisitor(const FVector& InStartPoint, const FVector& InDir, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer, EHitFlags InOutputFlags,
		const ChaosInterface::FQueryFilterData& InQueryFilterData, ICollisionQueryFilterCallbackBase& InQueryCallback, const ChaosInterface::FQueryDebugParams& InDebugParams)
		: TSQVisitor(InStartPoint, InDir, InHitBuffer, InOutputFlags, ChaosInterface::FSceneQueryCommonParams(InQueryCallback, InQueryFilterData, InDebugParams))
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TSQVisitor(const FTransform& InStartTM, const FVector& InDir, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer, EHitFlags InOutputFlags,
		const QueryGeometryType& InQueryGeom, const FAABB3& InQueryGeomWorldBounds, const ChaosInterface::FSceneQueryCommonParams& InCommonParams)
		: Dir(InDir)
		, OutputFlags(InOutputFlags)
		, bAnyHit(false)
		, DebugParams(InCommonParams.DebugParams)
		, HitFaceNormal(Chaos::FVec3::ZeroVector)
		, HitBuffer(InHitBuffer)
		, QueryFlags(InCommonParams.Flags)
		, QueryFilterData(InCommonParams.FilterData)
		, QueryGeom(&InQueryGeom)
		, QueryCallback(InCommonParams.GetQueryCallback())
		, StartTM(InStartTM)
		, QueryGeomWorldBounds(InQueryGeomWorldBounds)
		, QueryCenter(InQueryGeomWorldBounds.Center())
	{
		bAnyHit = QueryFlags & FChaosQueryFlag::eANY_HIT;
		//todo: check THitType is sweep
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use the overload that takes a ChaosInterface::FSceneQueryCommonParams instead")
	TSQVisitor(const FTransform& InStartTM, const FVector& InDir, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer, EHitFlags InOutputFlags,
		const ChaosInterface::FQueryFilterData& InQueryFilterData, ICollisionQueryFilterCallbackBase& InQueryCallback, const QueryGeometryType& InQueryGeom, const FAABB3& InQueryGeomWorldBounds, const ChaosInterface::FQueryDebugParams& InDebugParams)
		: TSQVisitor(InStartTM, InDir, InHitBuffer, InOutputFlags, InQueryGeom, InQueryGeomWorldBounds, ChaosInterface::FSceneQueryCommonParams(InQueryCallback, InQueryFilterData, InDebugParams))
	{
	}

	UE_DEPRECATED(5.8, "Use the overload that takes the QueryGeomWorldBounds")
	TSQVisitor(const FTransform& InStartTM, const FVector& InDir, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer, EHitFlags InOutputFlags,
		const ChaosInterface::FQueryFilterData& InQueryFilterData, ICollisionQueryFilterCallbackBase& InQueryCallback, const QueryGeometryType& InQueryGeom, const ChaosInterface::FQueryDebugParams& InDebugParams)
		: TSQVisitor(InStartTM, InDir, InHitBuffer, InOutputFlags, InQueryFilterData, InQueryCallback, InQueryGeom, InQueryGeom.CalculateTransformedBounds(InStartTM), InDebugParams)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TSQVisitor(const FTransform& InWorldTM, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer,
		const QueryGeometryType& InQueryGeom, const FAABB3& InQueryGeomWorldBounds, const ChaosInterface::FSceneQueryCommonParams& InCommonParams)
		: bAnyHit(false)
		, DebugParams(InCommonParams.DebugParams)
		, HitFaceNormal(Chaos::FVec3::ZeroVector)
		, HitBuffer(InHitBuffer)
		, QueryFlags(InCommonParams.Flags)
		, QueryFilterData(InCommonParams.FilterData)
		, QueryGeom(&InQueryGeom)
		, QueryCallback(InCommonParams.GetQueryCallback())
		, StartTM(InWorldTM)
		, QueryGeomWorldBounds(InQueryGeomWorldBounds)
		, QueryCenter(InQueryGeomWorldBounds.Center())
	{
		bAnyHit = QueryFlags & FChaosQueryFlag::eANY_HIT;
		//todo: check THitType is overlap
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use the overload that takes a ChaosInterface::FSceneQueryCommonParams instead")
	TSQVisitor(const FTransform& InWorldTM, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer,
		const ChaosInterface::FQueryFilterData& InQueryFilterData, ICollisionQueryFilterCallbackBase& InQueryCallback, const QueryGeometryType& InQueryGeom, const FAABB3& InQueryGeomWorldBounds, const ChaosInterface::FQueryDebugParams& InDebugParams)
		: TSQVisitor(InWorldTM, InHitBuffer, InQueryGeom, InQueryGeomWorldBounds, ChaosInterface::FSceneQueryCommonParams(InQueryCallback, InQueryFilterData, InDebugParams))
	{
	}

	UE_DEPRECATED(5.8, "Use the overload that takes the QueryGeomWorldBounds")
	TSQVisitor(const FTransform& InWorldTM, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer,
		const ChaosInterface::FQueryFilterData& InQueryFilterData, ICollisionQueryFilterCallbackBase& InQueryCallback, const QueryGeometryType& InQueryGeom, const ChaosInterface::FQueryDebugParams& InDebugParams)
		: TSQVisitor(InWorldTM, InHitBuffer, InQueryFilterData, InQueryCallback, InQueryGeom, InQueryGeom.CalculateTransformedBounds(InWorldTM), InDebugParams)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual ~TSQVisitor() = default;

	virtual bool Raycast(const Chaos::TSpatialVisitorData<TPayload>& Instance, Chaos::FQueryFastData& CurData) override
	{
		return Visit<ESQType::Raycast>(Instance, &CurData);
	}

	virtual bool Sweep(const Chaos::TSpatialVisitorData<TPayload>& Instance, Chaos::FQueryFastData& CurData) override
	{
		return Visit<ESQType::Sweep>(Instance, &CurData);
	}

	virtual bool Overlap(const Chaos::TSpatialVisitorData<TPayload>& Instance) override
	{
		return Visit<ESQType::Overlap>(Instance, nullptr);
	}

	virtual bool PrePreFilter(const TPayload& Payload) const override
	{
		return Payload.PrePreFilter(QueryFilterData);
	}

	/** Return a pointer to the payload on which we are querying the acceleration structure */
	virtual const void* GetQueryPayload() const override
	{
		return nullptr;
	}

	virtual bool HasBlockingHit() const override
	{
		return HitBuffer.HasBlockingHit();
	}

private:

	enum class ESQType
	{
		Raycast,
		Sweep,
		Overlap
	};

	TGeometryType* GetPayloadForThread(const TPayload& Payload)
	{
		if constexpr (bGTData)
		{
			return Payload.GetExternalGeometryParticle_ExternalThread();
		}
		else if constexpr (TPayload::bHasPayloadOnInternalThread)
		{
			return Payload.GetGeometryParticleHandle_PhysicsThread();
		}
		else
		{
			return nullptr;
		}
	}

	template <ESQType SQ>
	bool Visit(const Chaos::TSpatialVisitorData<TPayload>& Instance, Chaos::FQueryFastData* CurData)
	{
		//QUICK_SCOPE_CYCLE_COUNTER(SQVisit);
		TPayload Payload = Instance.Payload;

		//todo: add a check to ensure hitbuffer matches SQ type
		using namespace Chaos;
		TGeometryType* GeometryParticle = GetPayloadForThread(Payload);

		if (!GeometryParticle)
		{
			// This case handles particles created by the physics simulation without the main thread
			// being made aware of their creation. We have a PT particle but no external particle
			ensure(bGTData);

			return true;
		}

		// Detect and fix any dangling handles on the game and physics threads
		if constexpr (bGTData)
		{
			if (!(GeometryParticle->UniqueIdx() == Payload.UniqueIdx()))
			{
				UE_LOGF(LogChaos, Warning, "Query Dangling handle detected on game Thread. Payload Id: %d, Particle Id: %d", Payload.UniqueIdx().Idx, GeometryParticle->UniqueIdx().Idx);
				ensureMsgf(false, TEXT("Query dangling handle detected on game Thread. Payload Id: %d, Particle Id: %d"), Payload.UniqueIdx().Idx, GeometryParticle->UniqueIdx().Idx);
				return true;
			}
		}
		else
		{
			if (GeometryParticle->GetHandleIdx() == INDEX_NONE || !(GeometryParticle->UniqueIdx() == Payload.UniqueIdx()))
			{
				UE_LOGF(LogChaos, Warning, "Query Dangling handle detected on Physics Thread. Payload Id: %d, Particle Id: %d", Payload.UniqueIdx().Idx, GeometryParticle->UniqueIdx().Idx);
				ensureMsgf(false, TEXT("Query dangling handle detected on Physics Thread. Payload Id: %d, Particle Id: %d"), Payload.UniqueIdx().Idx, GeometryParticle->UniqueIdx().Idx);
				return true;
			}
		}

		const FShapesArray& Shapes = GeometryParticle->ShapesArray();

		const bool bTestShapeBounds = Shapes.Num() > 1;
		bool bContinue = true;

		const FRigidTransform3 ActorTM(GeometryParticle->GetX(), GeometryParticle->GetR());

#if CHAOS_DEBUG_DRAW
		bool bAllShapesIgnoredInPrefilter = true;
		bool bHitBufferIncreased = false;
#endif
		
		for (int32 ShapeIndex = 0; ShapeIndex < Shapes.Num(); ++ShapeIndex)
		{
			const TUniquePtr<FPerShapeData>& Shape = Shapes[ShapeIndex];

			const FImplicitObject* Geom = Shape->GetGeometry();

			CVD_TRACE_SCOPED_SCENE_QUERY_VISIT_HELPER(EChaosVDSceneQueryVisitorType::NarrowPhase, ActorTM, Payload.UniqueIdx().Idx, ShapeIndex, CurData);

			if (bTestShapeBounds)
			{
				FAABB3 InflatedWorldBounds;
				if (SQ == ESQType::Raycast)
				{
					InflatedWorldBounds = Shape->GetWorldSpaceShapeBounds();
				}
				else
				{
					// Transform to world bounds and get the proper half extent.
					const FVec3 WorldHalfExtent = QueryGeomWorldBounds.Extents() * 0.5f;

					InflatedWorldBounds = FAABB3(Shape->GetWorldSpaceShapeBounds().Min() - WorldHalfExtent, Shape->GetWorldSpaceShapeBounds().Max() + WorldHalfExtent);
				}
	
				if (SQ != ESQType::Overlap)
				{
					//todo: use fast raycast
					Chaos::FReal TmpTime, TmpExitTime;
					const FVec3 InflatedBoundsTraceStart = SQ == ESQType::Raycast ? StartPoint : QueryGeomWorldBounds.Center();
					if (!InflatedWorldBounds.RaycastFast(InflatedBoundsTraceStart, CurData->Dir, CurData->InvDir, CurData->bParallel, CurData->CurrentLength, CurData->InvCurrentLength, TmpTime, TmpExitTime))
					{
						CVD_SET_SQ_SHAPE_REJECT_REASON(EChaosVDSQVisitRejectReason::FailedFastBoundTest);
						continue;
					}
				}
				else
				{
					if (!InflatedWorldBounds.Contains(QueryCenter))
					{
						CVD_SET_SQ_SHAPE_REJECT_REASON(EChaosVDSQVisitRejectReason::FailedFastBoundTest);
						continue;
					}
				}
			}

			//TODO: use gt particles directly
			ECollisionQueryHitType HitType = QueryFlags & FChaosQueryFlag::ePREFILTER ? QueryCallback.PreFilter(QueryFilterData, *Shape, *GeometryParticle) : ECollisionQueryHitType::Block;

			if (HitType != ECollisionQueryHitType::None)
			{
#if CHAOS_DEBUG_DRAW
				bAllShapesIgnoredInPrefilter = false;
#endif

				//QUICK_SCOPE_CYCLE_COUNTER(SQNarrow);
				THitType Hit;
				Hit.Actor = GeometryParticle;
				Hit.Shape = Shape.Get();

				bool bHit = false;

				FVec3 WorldPosition{ 0.0f }, WorldNormal{0.0f};
				Chaos::FReal Distance = 0;	//not needed but fixes compiler warning for overlap
				int32 FaceIdx = INDEX_NONE;	//not needed but fixes compiler warning for overlap
				FVec3 FaceNormal = FVec3::ZeroVector;
				const bool bComputeMTD = !!((uint16)(OutputFlags.HitFlags & EHitFlags::MTD));

				if (SQ == ESQType::Raycast)
				{
					FVec3 LocalNormal;
					FVec3 LocalPosition;

					const FVec3 DirLocal = ActorTM.InverseTransformVectorNoScale(Dir);
					const FVec3 StartLocal = ActorTM.InverseTransformPositionNoScale(StartPoint);
					bHit = Geom->Raycast(StartLocal, DirLocal, CurData->CurrentLength, /*Thickness=*/0.0, Distance, LocalPosition, LocalNormal, FaceIdx);
					if (bHit)
					{
						WorldPosition = ActorTM.TransformPositionNoScale(LocalPosition);
						WorldNormal = ActorTM.TransformVectorNoScale(LocalNormal);
					}
				}
				else if (SQ == ESQType::Sweep && ensure(QueryGeom))
				{
					bHit = SweepQuery(*Geom, ActorTM, *QueryGeom, StartTM, CurData->Dir, CurData->CurrentLength, Distance, WorldPosition, WorldNormal, FaceIdx, FaceNormal, 0.f, bComputeMTD);
				}
				else if (SQ == ESQType::Overlap && ensure(QueryGeom))
				{
					if (bComputeMTD)
					{
						FMTDInfo MTDInfo;
						bHit = OverlapQuery(*Geom, ActorTM, *QueryGeom, StartTM, /*Thickness=*/0, &MTDInfo);
						if (bHit)
						{
							WorldNormal = MTDInfo.Normal;
							WorldPosition = MTDInfo.Position;
							Distance = -MTDInfo.Penetration;
						}
					}
					else
					{
						bHit = OverlapQuery(*Geom, ActorTM, *QueryGeom, StartTM, /*Thickness=*/0);
					}
				}

				if (bHit)
				{
					//QUICK_SCOPE_CYCLE_COUNTER(SQNarrowHit);

					bool bAcceptHit = true;
					if constexpr (SQ == ESQType::Sweep && std::is_same_v<THitType, ChaosInterface::FSweepHit>)
					{
						const THitType* CurrentHit = HitBuffer.GetCurrentHit();
						bool bIsCoLocatedHit = false;
						if (FaceIdx != INDEX_NONE || !FaceNormal.IsNearlyZero())
						{
							if (CurrentHit)
							{
								constexpr static FReal CoLocationEpsilon = 1e-6;
								const FReal DistDelta = FMath::Abs(Distance - CurrentHit->Distance);

								if (DistDelta < CoLocationEpsilon && !HitFaceNormal.IsNearlyZero())
								{
									bIsCoLocatedHit = true;
									// We already have a face hit from another triangle mesh - see if this one is better (more opposing the sweep)
									const FReal OldDot = FVec3::DotProduct(CurData->Dir, HitFaceNormal);
									const FReal NewDot = FVec3::DotProduct(CurData->Dir, FaceNormal);

									if (NewDot < OldDot)
									{
										// More opposing
										HitFaceNormal = FaceNormal;
									}
									else
									{
										CVD_SET_SQ_SHAPE_REJECT_REASON(EChaosVDSQVisitRejectReason::ColocatedHitHasWorseNormal);
										// This hit is co-located but has a worse normal
										bAcceptHit = false;
									}
								}
							}

							if (bAcceptHit)
							{
								// Record the new face normal
								HitFaceNormal = FaceNormal;
							}
						}
						// If we're not in a co-located hit, we need to make sure the new hit is actually better than the old.
						// For normal hits this is redundant due to clipping length.
						// When there is an initial overlaps though, we cannot clip length < 0, so we must check the returned distance.
						if (CurrentHit != nullptr && !bIsCoLocatedHit && CurrentHit->Distance < Distance)
						{
							bAcceptHit = false;
						}
					}

					if (bAcceptHit)
					{
						FillHitHelper(Hit, Distance, WorldPosition, WorldNormal, FaceIdx, bComputeMTD);

						if constexpr (std::is_base_of_v<ChaosInterface::FQueryHit, THitType> || std::is_base_of_v<ChaosInterface::FPTQueryHit, THitType>)
						{
							Hit.FaceNormal = HitFaceNormal;
						}

						HitType = QueryFlags & FChaosQueryFlag::ePOSTFILTER ? QueryCallback.PostFilter(QueryFilterData, Hit) : HitType;

						if (HitType != ECollisionQueryHitType::None)
						{
							//overlap never blocks
							const bool bBlocker = (HitType == ECollisionQueryHitType::Block || bAnyHit || HitBuffer.WantsSingleResult());
							HitBuffer.InsertHit(Hit, bBlocker);

							CVD_FILL_HIT_DATA_HELPER(Hit, HitType);

#if CHAOS_DEBUG_DRAW
							bHitBufferIncreased = true;
#endif

							if (bBlocker && SQ != ESQType::Overlap)
							{
								// When there is an initial overlap, MTD will return a negative distance. We need to clamp this to 0 as many checks don't work properly with a negative length.
								CurData->SetLength(FMath::Max((FReal)0., Distance));
								// If we have a zero length raycast, this is an initial overlap and there's no better result possible.
								// Note: We do not do this check with sweeps as some initial overlap sweeps are better than others (more negative MTD).
								if (CurData->CurrentLength == 0 && (SQ == ESQType::Raycast))
								{
									bContinue = false;
									break;
								}
							}

							if (bAnyHit)
							{
								bContinue = false;
								break;
							}
						}
						else
						{
							CVD_SET_SQ_SHAPE_REJECT_REASON(EChaosVDSQVisitRejectReason::PostFilter);
						}
					}
				}
				else
				{
					CVD_SET_SQ_SHAPE_REJECT_REASON(EChaosVDSQVisitRejectReason::NoHit);
				}
			}
			else
			{
				CVD_SET_SQ_SHAPE_REJECT_REASON(EChaosVDSQVisitRejectReason::PreFilter);
			}
		}

#if CHAOS_DEBUG_DRAW
		if (DebugParams.IsDebugQuery() && ChaosSQDrawDebugVisitorQueries)
		{
			DebugDraw<SQ>(Instance, CurData, bAllShapesIgnoredInPrefilter, bHitBufferIncreased);
		}
#endif

		return bContinue;
	}

#if CHAOS_DEBUG_DRAW

	void DebugDrawPayloadImpl(const TPayload& Payload, const bool bExternal, const bool bHit, decltype(&TPayload::DebugDraw))
	{
		Payload.DebugDraw(bExternal, bHit);
	}
	void DebugDrawPayloadImpl(const TPayload& Payload, const bool bExternal, const bool bHit, ...)
	{}
	void DebugDrawPayload(const TPayload& Payload, const bool bExternal, const bool bHit)
	{
		DebugDrawPayloadImpl(Payload, bExternal, bHit, 0);
	}

	template <ESQType SQ>
	void DebugDraw(const Chaos::TSpatialVisitorData<TPayload>& Instance, const Chaos::FQueryFastData* CurData, const bool bPrefiltered, const bool bHit)
	{
		if (SQ == ESQType::Raycast)
		{
			const FVector EndPoint = StartPoint + (Dir * CurData->CurrentLength);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(StartPoint, EndPoint, 5.f, bHit ? FColor::Red : FColor::Green);
		}
		else if (SQ == ESQType::Overlap)
		{
			Chaos::DebugDraw::DrawShape(StartTM, QueryGeom, nullptr, bHit ? FColor::Red : FColor::Green, 0.0f);
		}

		if (Instance.bHasBounds)
		{
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Instance.Bounds.Center(), Instance.Bounds.Extents(), FQuat::Identity, bHit ? FColor(100, 50, 50) : FColor(50, 100, 50), false, -1.f, 0, 0.f);
		}

		if (!bPrefiltered)
		{
			DebugDrawPayload(Instance.Payload, DebugParams.bExternalQuery, bHit);
		}
	}
#endif

	const FVector StartPoint = FVector::ZeroVector;
	const FVector Dir = FVector::ZeroVector;
	FHitFlags OutputFlags;
	bool bAnyHit;
	const ChaosInterface::FQueryDebugParams DebugParams;
	Chaos::FVec3 HitFaceNormal;
	ChaosInterface::FSQHitBuffer<THitType>& HitBuffer;
	const FChaosQueryFlags QueryFlags;
	const Chaos::Filter::FQueryFilterData QueryFilterData;
	const QueryGeometryType* QueryGeom = nullptr;
	ICollisionQueryFilterCallback& QueryCallback;
	const FTransform StartTM;
	FAABB3 QueryGeomWorldBounds = FAABB3::ZeroAABB();
	FVec3 QueryCenter = FVec3::ZeroVector;
};


template <typename QueryGeometryType, typename TPayload, typename THitType, bool bGTData = true>
struct TBPVisitor : public Chaos::ISpatialVisitor<TPayload, Chaos::FReal>
{
	using TGeometryType = std::conditional_t<bGTData, Chaos::FGeometryParticle, Chaos::FGeometryParticleHandle>;

	TBPVisitor(const FTransform& InWorldTM, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer,
		const QueryGeometryType& InQueryGeom, const ChaosInterface::FSceneQueryCommonParams& InCommonParams)
		: HalfExtents(InQueryGeom.BoundingBox().Extents() * 0.5)
		, bAnyHit(false)
		, DebugParams(InCommonParams.DebugParams)
		, HitBuffer(InHitBuffer)
		, QueryFlags(InCommonParams.Flags)
		, QueryFilterData(InCommonParams.FilterData)
		, QueryGeom(&InQueryGeom)
		, QueryCallback(InCommonParams.GetQueryCallback())
		, StartTM(InWorldTM)
	{
		bAnyHit = QueryFlags & FPhysicsQueryFlag::eANY_HIT;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use the overload that takes a ChaosInterface::FSceneQueryCommonParams instead")
	TBPVisitor(const FTransform& InWorldTM, ChaosInterface::FSQHitBuffer<THitType>& InHitBuffer,
		const ChaosInterface::FQueryFilterData& InQueryFilterData, ICollisionQueryFilterCallbackBase& InQueryCallback, const QueryGeometryType& InQueryGeom, const ChaosInterface::FQueryDebugParams& InDebugParams)
		: TBPVisitor(InWorldTM, InHitBuffer, InQueryGeom, ChaosInterface::FSceneQueryCommonParams(InQueryCallback, InQueryFilterData, InDebugParams))
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual ~TBPVisitor() = default;

	virtual bool Overlap(const Chaos::TSpatialVisitorData<TPayload>& Instance) override
	{
		return Visit<ESQType::Overlap>(Instance, nullptr);
	}

	virtual bool Raycast(const Chaos::TSpatialVisitorData<TPayload>& Instance, Chaos::FQueryFastData& CurData) override
	{
		ensure(false);
		return false;
	}

	virtual bool Sweep(const Chaos::TSpatialVisitorData<TPayload>& Instance, Chaos::FQueryFastData& CurData) override
	{
		ensure(false);
		return false;
	}

	virtual bool PrePreFilter(const TPayload& Payload) const override
	{
		return Payload.PrePreFilter(QueryFilterData);
	}

	/** Return a pointer to the payload on which we are querying the acceleration structure */
	virtual const void* GetQueryPayload() const override
	{
		return nullptr;
	}

private:

	enum class ESQType
	{
		Raycast,
		Sweep,
		Overlap
	};

	TGeometryType* GetPayloadForThread(const TPayload& Payload)
	{
		if constexpr (bGTData)
		{
			return Payload.GetExternalGeometryParticle_ExternalThread();
		}
		else if constexpr (TPayload::bHasPayloadOnInternalThread)
		{
			return Payload.GetGeometryParticleHandle_PhysicsThread();
		}
		else
		{
			return nullptr;
		}
	}

	template <ESQType SQ>
	bool Visit(const Chaos::TSpatialVisitorData<TPayload>& Instance, Chaos::FQueryFastData* CurData)
	{
		bool bContinue = true;
		TPayload Payload = Instance.Payload;
		//todo: add a check to ensure hitbuffer matches SQ type
		using namespace Chaos;
		TGeometryType* GeometryParticle = GetPayloadForThread(Payload);
		if (!GeometryParticle)
		{
			// This case handles particles created by the physics simulation without the main thread
			// being made aware of their creation. We have a PT particle but no external particle
			ensure(bGTData);
			return true;
		}
		const FShapesArray& Shapes = GeometryParticle->ShapesArray();
		THitType Hit;
		Hit.Actor = GeometryParticle;

		for (int32 ShapeIndex = 0; ShapeIndex < Shapes.Num(); ++ShapeIndex)
		{
			const TUniquePtr<FPerShapeData>& Shape = Shapes[ShapeIndex];

			CVD_TRACE_SCOPED_SCENE_QUERY_VISIT_HELPER(EChaosVDSceneQueryVisitorType::BroadPhase, FTransform(GeometryParticle->GetR(), GeometryParticle->GetX()), Payload.UniqueIdx().Idx, ShapeIndex, CurData);

			ECollisionQueryHitType HitType = QueryFlags & FPhysicsQueryFlag::ePREFILTER ? QueryCallback.PreFilter(QueryFilterData, *Shape, *GeometryParticle) : ECollisionQueryHitType::Block;
			if (HitType != ECollisionQueryHitType::None)
			{
				const bool bBlocker = (HitType == ECollisionQueryHitType::Block || bAnyHit || HitBuffer.WantsSingleResult());
				Hit.Shape = Shape.Get();
				HitBuffer.InsertHit(Hit, bBlocker);

				CVD_FILL_HIT_DATA_HELPER(Hit, HitType);

				if (bAnyHit)
				{
					bContinue = false;
				}
				break;
			}
			else
			{
				CVD_SET_SQ_SHAPE_REJECT_REASON(EChaosVDSQVisitRejectReason::PreFilter);
			}
		}
		return bContinue;
	}

	const FVector StartPoint = FVector::ZeroVector;
	const FVector Dir = FVector::ZeroVector;
	const FVector HalfExtents = FVector::ZeroVector;
	FHitFlags OutputFlags;
	bool bAnyHit;
	const ChaosInterface::FQueryDebugParams DebugParams;
	ChaosInterface::FSQHitBuffer<THitType>& HitBuffer;
	const FChaosQueryFlags QueryFlags;
	const Chaos::Filter::FQueryFilterData QueryFilterData;
	const QueryGeometryType* QueryGeom;
	ICollisionQueryFilterCallback& QueryCallback;
	const FTransform StartTM;
};


template <typename QueryGeomType, typename TSweepHit, typename TPayload>
void SweepHelper(const QueryGeomType& QueryGeom, const Chaos::ISpatialAcceleration<TPayload, Chaos::FReal, 3>& SpatialAcceleration, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<TSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& InCommonParams)
{
	using namespace Chaos;
	using namespace ChaosInterface;

	const FAABB3 Bounds = QueryGeom.CalculateTransformedBounds(StartTM);
	// There are cases where an invalid shape comes in which produces an invalid bounds. In this case, nan will enter causing the entire tree to be queried which will timeout.
	const bool bIsValid = !Bounds.Min().ContainsNaN() && !Bounds.Max().ContainsNaN();
	if (ensure(bIsValid))
	{
		TSQVisitor<QueryGeomType, TPayload, TSweepHit, std::is_same<TSweepHit, FSweepHit>::value> SweepVisitor(StartTM, Dir, HitBuffer, OutputFlags, QueryGeom, Bounds, InCommonParams);

		HitBuffer.IncFlushCount();

		// NOTE: Previously, we'd attempt to perform a sweep as an overlap if the DeltaMagnitude was 0. 
		// This caused various discrepancies as some of the out info couldn't be computed. Instead, sweep has been updated to handle 0 length sweeps.
		const FVector HalfExtents = Bounds.Extents() * 0.5f;
		SpatialAcceleration.Sweep(Bounds.GetCenter(), Dir, DeltaMagnitude, HalfExtents, SweepVisitor);

		HitBuffer.DecFlushCount();
	}
}


template <typename QueryGeomType, typename TOverlapHit, typename TPayload>
void OverlapHelper(const QueryGeomType& QueryGeom, const Chaos::ISpatialAcceleration<TPayload, Chaos::FReal, 3>& SpatialAcceleration, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<TOverlapHit>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& InCommonParams)
{
	using namespace Chaos;
	using namespace ChaosInterface;

	const FAABB3 Bounds = QueryGeom.CalculateTransformedBounds(GeomPose);
	// There are cases where an invalid shape comes in which produces an invalid bounds. In this case, nan will enter causing the entire tree to be queried which will timeout.
	const bool bIsValid = !Bounds.Min().ContainsNaN() && !Bounds.Max().ContainsNaN();
	if (ensure(bIsValid))
	{
		HitBuffer.IncFlushCount();

		bool bSkipNarrowPhase = InCommonParams.Flags & FPhysicsQueryFlag::eSKIPNARROWPHASE;

		constexpr bool bGTData = std::is_same<TOverlapHit, FOverlapHit>::value;
		if (bSkipNarrowPhase)
		{
			TBPVisitor<QueryGeomType, TPayload, TOverlapHit, bGTData> OverlapVisitor(GeomPose, HitBuffer, QueryGeom, InCommonParams);
			SpatialAcceleration.Overlap(Bounds, OverlapVisitor);
		}
		else
		{
			TSQVisitor<QueryGeomType, TPayload, TOverlapHit, bGTData> OverlapVisitor(GeomPose, HitBuffer, QueryGeom, Bounds, InCommonParams);
			SpatialAcceleration.Overlap(Bounds, OverlapVisitor);
		}
		HitBuffer.DecFlushCount();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <typename QueryGeomType, typename TSweepHit, typename TPayload>
UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
void SweepHelper(const QueryGeomType& QueryGeom, const Chaos::ISpatialAcceleration<TPayload, Chaos::FReal, 3>& SpatialAcceleration, const FTransform& StartTM, const FVector& Dir, const float DeltaMagnitude, ChaosInterface::FSQHitBuffer<TSweepHit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams)
{
	const ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback, QueryFilterData, DebugParams);
	SweepHelper(QueryGeom, SpatialAcceleration, StartTM, Dir, DeltaMagnitude, HitBuffer, OutputFlags, CommonParams);
}

template <typename QueryGeomType, typename TOverlapHit, typename TPayload>
UE_DEPRECATED(5.8, "Use the overload that takes ChaosInterface::FSceneQueryCommonParams instead")
void OverlapHelper(const QueryGeomType& QueryGeom, const Chaos::ISpatialAcceleration<TPayload, Chaos::FReal, 3>& SpatialAcceleration, const FTransform& GeomPose, ChaosInterface::FSQHitBuffer<TOverlapHit>& HitBuffer, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase& QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams)
{
	const ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback, QueryFilterData, DebugParams);
	OverlapHelper(QueryGeom, SpatialAcceleration, GeomPose, HitBuffer, CommonParams);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
