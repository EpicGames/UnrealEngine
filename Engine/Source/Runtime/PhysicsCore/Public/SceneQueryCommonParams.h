// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsInterfaceTypesCore.h"
#include "ChaosInterfaceWrapperCore.h"
#include "Chaos/CollisionFilterData.h"

class ICollisionQueryFilterCallback;

namespace ChaosInterface
{
	struct FSceneQueryCommonParams
	{
		using FQueryFilterData = Chaos::Filter::FQueryFilterData;

		PHYSICSCORE_API explicit FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback);
		PHYSICSCORE_API explicit FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback, const FQueryFilterData& InFilterData, FChaosQueryFlags InFlags, const FQueryDebugParams& InDebugParams = {});
		PHYSICSCORE_API explicit FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback, const FQueryFilterData& InFilterData, EQueryFlags InFlags, const FQueryDebugParams& InDebugParams = {});

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Temporary constructor for deprecated code.
		UE_INTERNAL PHYSICSCORE_API explicit FSceneQueryCommonParams(ICollisionQueryFilterCallback& InQueryCallback, const FChaosQueryFilterData& InFilterData, const FQueryDebugParams& InDebugParams = {});
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		PHYSICSCORE_API ICollisionQueryFilterCallback& GetQueryCallback() const;

		FQueryFilterData FilterData;
		FChaosQueryFlags Flags;
		FQueryDebugParams DebugParams;

	private:
		ICollisionQueryFilterCallback* QueryCallback = nullptr;
	};
} // namespace ChaosInterface
